// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class DefaultInstallBundleManager_WinGDK : DefaultInstallBundleManager
	{
		public DefaultInstallBundleManager_WinGDK(ReadOnlyTargetRules Target) : base(Target)
		{
			PrivateDependencyModuleNames.Add("GDKPackageManifest");

			PrivateIncludePathModuleNames.AddRange(
				new string[] {
					"GRDK",
					"GDKRuntime",
				}
			);
		}
	}
}
