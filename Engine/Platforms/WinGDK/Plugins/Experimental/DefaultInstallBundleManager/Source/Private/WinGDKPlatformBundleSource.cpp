// Copyright Epic Games, Inc. All Rights Reserved.

#include "InstallBundleManagerUtil.h"

#if WITH_PLATFORM_INSTALL_BUNDLE_SOURCE
#include "HAL/PlatformMisc.h"
#include "InstallBundleSourcePlatformChunkInstall.h"
namespace InstallBundleManagerUtil
{
	IPlatformChunkInstall* GetPlatformChunkInstall()
	{
		IPlatformChunkInstall* PlatformChunkInstall = FPlatformMisc::GetPlatformChunkInstall();
		if (PlatformChunkInstall && PlatformChunkInstall->SupportsBundleSource())
		{
			return PlatformChunkInstall;
		}

		return nullptr;
	}

	TSharedPtr<IInstallBundleSource> MakePlatformBundleSource()
	{
		IPlatformChunkInstall* PlatformChunkInstall = GetPlatformChunkInstall();
		if (PlatformChunkInstall)
		{
			return MakeShared<FInstallBundleSourcePlatformChunkInstall>(PlatformChunkInstall);
		}

		return nullptr;
	}
}
#endif // WITH_PLATFORM_INSTALL_BUNDLE_SOURCE
#if WITH_PLATFORM_DLC_INSTALL_BUNDLE_SOURCE
#include "PlatformDLCModule.h"
namespace InstallBundleManagerUtil
{
	TSharedPtr<IPlatformDLC> GetPlatformDLC()
	{
		IPlatformDLCModule* PlatformDLCModule = FModuleManager::LoadModulePtr<IPlatformDLCModule>(TEXT("PlatformDLC"));
		if (PlatformDLCModule != nullptr)
		{
			return PlatformDLCModule->GetPlatformDLC();
		}

		return nullptr;
	}
}
#endif // WITH_PLATFORM_DLC_INSTALL_BUNDLE_SOURCE
