{
    "FileVersion" : 3,
    "Version" : 1,
    "VersionName" : "1.0",
    "FriendlyName" : "Default Install Bundle Manager",
    "Description" : "Default engine handler for downloading, patching, and mounting content bundles while the game is running.",
    "CreatedBy" : "Epic Games, Inc.",
	"CreatedByURL" : "https://epicgames.com",
	"DocsURL" : "",
	"MarketplaceURL" : "",
	"SupportURL" : "",
	"EnabledByDefault" : false,
	"CanContainContent" : false,
	"IsBetaVersion" : false,
    "Installed" : false,
    "Modules": [
		{
			"Name": "DefaultInstallBundleManager",
			"Type": "Runtime",
			"LoadingPhase": "PreEarlyLoadingScreen"
		}
    ],
	"Plugins": [
		{
			"Name": "OnlineSubsystem",
			"Enabled": true
		},
		{
			"Name": "OnlineFramework",
			"Enabled": true
		},
		{
			"Name": "PlatformDLC",
			"Enabled": true
		}
	],
	"LocalizationTargets": [
		{
			"Name": "DefaultInstallBundleManager",
			"LoadingPolicy": "Always"
		}
	]
}