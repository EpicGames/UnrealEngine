// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

// This module must be loaded "PreLoadingScreen" in the .uproject file, otherwise it will not hook in time!

public class DefaultInstallBundleManager : ModuleRules
{
	public DefaultInstallBundleManager(ReadOnlyTargetRules Target) : base(Target)
	{
		PublicDependencyModuleNames.AddRange(
			new string[] {
				"InstallBundleManager",
				"PatchCheck",
			}
		);

		PrivateDependencyModuleNames.AddRange(
			new string[] {
				"AnalyticsET",
				"ApplicationCore",
				"Core",
				"Json",
				"PakFile",
				"RenderCore",
				"IoStoreOnDemandCore",
				"PlatformDLC",
			}
		);
	}
}
