// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ProfilingDebugging/CsvProfiler.h"

CSV_DECLARE_CATEGORY_MODULE_EXTERN(INSTALLBUNDLEMANAGER_API, InstallBundleManager);

#define LOG_INSTALL_BUNDLE_MAN(Verbosity, Format, ...) UE_LOG(LogDefaultInstallBundleManager, Verbosity, Format, ##__VA_ARGS__)

#define LOG_INSTALL_BUNDLE_MAN_OVERRIDE(InVerbosityOverride, Verbosity, Format, ...) \
{ \
	if((ELogVerbosity::Verbosity & ELogVerbosity::VerbosityMask) < ELogVerbosity::Display) \
	{ \
		UE_LOG(LogDefaultInstallBundleManager, Verbosity, Format, ##__VA_ARGS__) \
	} \
	else \
	{ \
		ELogVerbosity::Type VerbosityOverride = (ELogVerbosity::Type)(InVerbosityOverride & ELogVerbosity::VerbosityMask); \
		if (VerbosityOverride <= ELogVerbosity::Verbosity) \
		{ \
			UE_LOG(LogDefaultInstallBundleManager, Verbosity, Format, ##__VA_ARGS__) \
		} \
		else \
		{ \
			switch (VerbosityOverride) \
			{ \
			case ELogVerbosity::Log: \
				UE_LOG(LogDefaultInstallBundleManager, Log, Format, ##__VA_ARGS__) \
				break; \
			case ELogVerbosity::Verbose: \
				UE_LOG(LogDefaultInstallBundleManager, Verbose, Format, ##__VA_ARGS__) \
				break; \
			case ELogVerbosity::VeryVerbose: \
				UE_LOG(LogDefaultInstallBundleManager, VeryVerbose, Format, ##__VA_ARGS__) \
				break; \
			default: \
				checkf(false, TEXT("Unkown verbosity override")); \
				UE_LOG(LogDefaultInstallBundleManager, Verbosity, Format, ##__VA_ARGS__) \
				break; \
			} \
		} \
	} \
}
