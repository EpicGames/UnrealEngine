// Copyright Epic Games, Inc. All Rights Reserved.

#include "InstallBundleSourceBulk.h"

#include "Algo/AnyOf.h"
#include "DefaultInstallBundleManagerPrivate.h"
#include "HAL/PlatformFileManager.h"
#include "HAL/PlatformFile.h"
#include "IPlatformFilePak.h"
#include "InstallBundleManagerInterface.h"
#include "InstallBundleManagerUtil.h"
#include "Misc/CommandLine.h"
#include "Misc/Parse.h"
#include "Misc/FileHelper.h"
#include "Misc/ConfigCacheIni.h"
#include "Internationalization/Regex.h"
#include "Misc/App.h"
#include "Async/ParallelFor.h"

#define LOG_SOURCE_BULK(Verbosity, Format, ...) LOG_INSTALL_BUNDLE_MAN(Verbosity, TEXT("InstallBundleSourceBulk: ") Format, ##__VA_ARGS__)

#define LOG_SOURCE_BULK_OVERRIDE(VerbosityOverride, Verbosity, Format, ...) LOG_INSTALL_BUNDLE_MAN_OVERRIDE(VerbosityOverride, Verbosity, TEXT("InstallBundleSourceBulk: ") Format, ##__VA_ARGS__)

//Helper class used to load/save BulkBuildBundle information from/to disk
class FBulkBuildBundleMapJsonInfo
	: public FJsonSerializable
{
public:

	//Simple Wrapper to hold bundle name in a way we can serialize it to/from a map with JSON_SERIALIZE_MAP_SERIALIZABLE
	class FJsonBundleNameWrapper
		: public FJsonSerializable
	{
		public:
			BEGIN_JSON_SERIALIZER
				JSON_SERIALIZE("BundleName", BundleName);
			END_JSON_SERIALIZER

			FString BundleName;
			
			FJsonBundleNameWrapper()
				: BundleName()
			{}
	};
	
	BEGIN_JSON_SERIALIZER
		JSON_SERIALIZE_MAP_SERIALIZABLE("BulkBuildBundleByFileMap", BulkBuildBundleByFileMap, FJsonBundleNameWrapper);
	END_JSON_SERIALIZER

	bool LoadFromFile(FStringView FilePath)
	{
		FString JSONStringOnDisk;
		if (FPaths::FileExists(FilePath.GetData()))
		{
			FFileHelper::LoadFileToString(JSONStringOnDisk, FilePath.GetData());
		}

		if (!JSONStringOnDisk.IsEmpty())
		{
			return ensureAlwaysMsgf(
				FromJson(JSONStringOnDisk),
				TEXT("Invalid JSON found while parsing BulkBuildBundleMapInfo from JSON: %s loaded from file:%.*s"), 
				*JSONStringOnDisk,
				FilePath.Len(), FilePath.GetData());
		}

		return false;
	}

	bool SaveToFile(FStringView FilePath)
	{
		return ensureAlwaysMsgf(
				FFileHelper::SaveStringToFile(ToJson(), FilePath.GetData()),
				TEXT("Error saving Json output of FBulkBuildBundleMapInfo to %.*s"),
				FilePath.Len(),FilePath.GetData());
	}

	//Takes in a list of files found on disk and uses our parsed BulkBuildBundleByFileMap to fill out the OutBulkBuildBundles Map.
	//Removes all found entries from the OutBundleFileList that were successfully processed.
	bool AppendEntriesToBulkBuildBundleMap(TArray<FString>& InOutBundleFileList, TMap<FName, TArray<FString>>& InOutBulkBuildBundles)
	{
		int32 OriginalBulkBuildBundleNum = InOutBulkBuildBundles.Num();

		if (InOutBundleFileList.Num() == 0)
		{
			return false;
		}

		for (int FileIndex = 0; FileIndex < InOutBundleFileList.Num();)
		{
			FString& FileInList = InOutBundleFileList[FileIndex];
			FJsonBundleNameWrapper* FoundBundleNameString = BulkBuildBundleByFileMap.Find(FileInList);
			if (FoundBundleNameString)
			{
				FName BundleName(*(FoundBundleNameString->BundleName));
				TArray<FString>& FoundFileList = InOutBulkBuildBundles.FindOrAdd(BundleName);
				FoundFileList.AddUnique(FileInList);

				//Remove and don't increment FileIndex so that we re-check this swapped index next pass if valid
				InOutBundleFileList.RemoveAtSwap(FileIndex);
			}
			else
			{
				++FileIndex;
			}
		}

		//return true if we've added values to OutBulkBuildBundles
		return (InOutBulkBuildBundles.Num() > OriginalBulkBuildBundleNum);
	}

	bool IsEmpty()
	{
		return (BulkBuildBundleByFileMap.Num() == 0);
	}

	//Gets the path to use for the BulkBuildInfo file if its present in the cooked data on device
	static FStringView GetBulkBuildBundleInfoCookedPath()
	{
		static FString CookedPath = FPaths::Combine(FPaths::ProjectContentDir(), TEXT("BulkBuildMeta"), TEXT("CachedBuilkBuildBuildInfo.json"));
		return CookedPath;
	}

	//Gets the path to use for the BulkBuildInfo file if its cached locally on the device
	static FStringView GetBulkBuildBundleInfoLocalCachedPath()
	{
		static FString LocalCachedPath = FPaths::Combine(FPaths::ProjectUserDir(), TEXT("Saved"), TEXT("BulkBuildMeta"), TEXT("CachedBuilkBuildBuildInfo.json"));
		return LocalCachedPath;
	}

	//Constructor to generate BulkBuildBundleByFileMap from BulkBuildBundlesIn, useful for loading the BulkBuildBundle info from
	//memory and then saving it out to file
	FBulkBuildBundleMapJsonInfo(const TMap<FName, TArray<FString>>& BulkBuildBundlesIn)
		: FBulkBuildBundleMapJsonInfo()
	{
		for(const TPair<FName, TArray<FString>>& BundlePair : BulkBuildBundlesIn)
		{
			for (const FString& FileName : BundlePair.Value)
			{
				FJsonBundleNameWrapper& FoundBundleName = BulkBuildBundleByFileMap.FindOrAdd(FileName);
				FoundBundleName.BundleName = BundlePair.Key.ToString();
			}
		}
	}

	FBulkBuildBundleMapJsonInfo()
		: BulkBuildBundleByFileMap()
	{};

private:
	//Serialized BulkBuildBundle (FileName)->(Bundle FString version of FName) information
	TMap<FString, FJsonBundleNameWrapper> BulkBuildBundleByFileMap;
};

FInstallBundleSourceBulk::FInstallBundleSourceBulk()
{
	TickHandle = FTSTicker::GetCoreTicker().AddTicker(FTickerDelegate::CreateRaw(this, &FInstallBundleSourceBulk::Tick));
}

FInstallBundleSourceBulk::~FInstallBundleSourceBulk()
{
	FTSTicker::RemoveTicker(TickHandle);
	TickHandle.Reset();

	InstallBundleUtil::CleanupInstallBundleAsyncIOTasks(InitAsyncTasks);
}

bool FInstallBundleSourceBulk::Tick(float dt)
{
	QUICK_SCOPE_CYCLE_COUNTER(STAT_FInstallBundleSourceBulk_Tick);

	TickInit();

	// Check for Init Task Completion
	InstallBundleUtil::FinishInstallBundleAsyncIOTasks(InitAsyncTasks);

	return true;
}

void FInstallBundleSourceBulk::TickInit()
{
	if (!OnInitCompleteCallback.IsBound())
		return;

	while (InitState == EInstallBundleManagerInitState::NotInitialized && InitStepResult == EAsyncInitStepResult::Done)
	{
		if (InitResult != EInstallBundleManagerInitResult::OK)
		{
			if (LastInitStep != InitStep)
			{
				// Only fire init analytic for failures the first time we retry
				AsyncInit_FireInitAnlaytic();
				LastInitStep = InitStep;
			}

			if(bRetryInit)
			{
				LOG_SOURCE_BULK(Warning, TEXT("Retrying initialization after %s"), LexToString(InitResult));
				InitResult = EInstallBundleManagerInitResult::OK;
				bRetryInit = false;
			}
			else
			{
				LOG_SOURCE_BULK(Warning, TEXT("Initialization Failed - %s"), LexToString(InitResult));
				InitState = EInstallBundleManagerInitState::Failed;
				break;
			}
		}
		else
		{
			LastInitStep = InitStep;
			++InstallBundleUtil::CastAsUnderlying(InitStep);
		}

		switch (InitStep)
		{
		case EAsyncInitStep::None:
			LOG_SOURCE_BULK(Fatal, TEXT("Trying to use init state None"));
			break;
		case EAsyncInitStep::MakeBundlesForBulkBuild:
			AsyncInit_MakeBundlesForBulkBuild();
			break;
		case EAsyncInitStep::Finishing:
			AsyncInit_FireInitAnlaytic();
			InitState = EInstallBundleManagerInitState::Succeeded;
			break;
		default:
			LOG_SOURCE_BULK(Fatal, TEXT("Unknown Init Step %s"), LexToString(InitStep));
			break;
		}
	}

	if (InitState == EInstallBundleManagerInitState::Succeeded || InitState == EInstallBundleManagerInitState::Failed)
	{
		FInstallBundleSourceAsyncInitInfo InitInfo;
		InitInfo.Result = InitResult;
		InitInfo.bShouldUseFallbackSource = false;

		OnInitCompleteCallback.Execute(AsShared(), MoveTemp(InitInfo));
		OnInitCompleteCallback = nullptr;
	}
}

void FInstallBundleSourceBulk::AsyncInit_FireInitAnlaytic()
{
	LOG_SOURCE_BULK(Display, TEXT("Fire Init Analytic: %s"), LexToString(InitResult));

	InstallBundleManagerAnalytics::FireEvent_InitBundleSourceBulkComplete(AnalyticsProvider.Get(), LexToString(InitResult));

	InitStepResult = EAsyncInitStepResult::Done;
}

void FInstallBundleSourceBulk::AsyncInit_MakeBundlesForBulkBuild()
{
	LOG_SOURCE_BULK(Display, TEXT("Making Bundles for Bulk Build"));

	InitStepResult = EAsyncInitStepResult::Waiting;

	TArray<FString> PakSearchDirs;
	FPakPlatformFile::GetPakFolders(FCommandLine::Get(), PakSearchDirs);

	//Get setting for if we limit our file list to only .pak files
	bool bOnlyGatherPaksInBulkData = false;
	if (!GConfig->GetBool(TEXT("InstallBundleSource.Bulk.MiscSettings"), TEXT("bOnlyGatherPaksInBulkData"), bOnlyGatherPaksInBulkData, GInstallBundleIni))
	{
		bOnlyGatherPaksInBulkData = false;
	}

	TSharedPtr<TArray<FString>, ESPMode::ThreadSafe> FoundFiles = MakeShared<TArray<FString>, ESPMode::ThreadSafe>();
	InstallBundleUtil::StartInstallBundleAsyncIOTask(InitAsyncTasks,
	[FoundFiles, PakSearchDirs=MoveTemp(PakSearchDirs), ContentDir=FPaths::ProjectContentDir(), bOnlyGatherPaksInBulkData]()
	{
		// Custom visitor because we need to deal with multiple extensions
		class FFindBulkFilesVisitor : public IPlatformFile::FDirectoryVisitor
		{
		public:
			FRWLock FoundFilesLock;
			TArray<FString>& FoundFiles;
			bool bOnlyGatherPaksInBulkData;
			FFindBulkFilesVisitor(TArray<FString>& InFoundFiles, bool bInOnlyGatherPaksInBulkData)
				: IPlatformFile::FDirectoryVisitor(EDirectoryVisitorFlags::ThreadSafe)
				, FoundFiles(InFoundFiles)
				, bOnlyGatherPaksInBulkData(bInOnlyGatherPaksInBulkData)
			{}

			virtual bool Visit(const TCHAR* FilenameOrDirectory, bool bIsDirectory) override
			{
				const FStringView FileExtensions[] = { TEXTVIEW(".pak"), TEXTVIEW(".uondemandtoc") };

				if (!bIsDirectory)
				{
					bool bFound = !bOnlyGatherPaksInBulkData;
					if (!bFound)
					{
						FStringView PathExtension = FPathViews::GetExtension(FilenameOrDirectory, true);
						for (FStringView Extension : FileExtensions)
						{
							if (PathExtension == Extension)
							{
								bFound = true;
								break;
							}
						}
					}

					if (bFound)
					{
						FString FileName(FilenameOrDirectory);
						FRWScopeLock ScopeLock(FoundFilesLock, SLT_Write);
						FoundFiles.Emplace(MoveTemp(FileName));
					}
				}

				return true;
			}
		};

		FFindBulkFilesVisitor Visitor(*FoundFiles, bOnlyGatherPaksInBulkData);
		for (const FString& SearchDir : PakSearchDirs)
		{
			FPlatformFileManager::Get().GetPlatformPhysical().IterateDirectoryRecursively(*SearchDir, Visitor);
		}
	},
	[this, FoundFiles]()
	{
		//First load any existing BulkBuildBundleIni entries and use those to sort FoundFiles into bundles
		const bool bHasAnyFilesToParse = FoundFiles.IsValid() && (FoundFiles->Num() > 0);
		// TODO: this cache is incompatible with bundle subsets. Its also unclear if its still needed, so I'm disabling it for now
		// bool bDidLoadAllFilesFromMetadata = bHasAnyFilesToParse ? TryLoadBulkBuildBundleMetadata(*FoundFiles, BulkBuildBundles) : false;
		bool bDidLoadAllFilesFromMetadata = false;

		// Prune out any remaining paks that were mounted at startup
		for (int i = 0; i < FoundFiles->Num();)
		{
			const FString& File = (*FoundFiles)[i];
			if (File.MatchesWildcard(FPakPlatformFile::GetMountStartupPaksWildCard()))
			{
				FoundFiles->RemoveAtSwap(i);
			}
			else
			{
				++i;
			}
		}

		//Still files left that weren't in the Metadata or StartupPaksWildcard, so we have files that will be manually parsed
		if (FoundFiles->Num() > 0)
		{
			bDidLoadAllFilesFromMetadata = false;
		}

		LOG_SOURCE_BULK(Display, TEXT("Loaded %d Bundle Information from BulkBuildBundles Cache. %d Files Remaining."), BulkBuildBundles.Num(), FoundFiles->Num());

		TArray<FString> SectionNames;
		const FConfigFile* InstallBundleConfig = GConfig->FindConfigFile(GInstallBundleIni);
		if (InstallBundleConfig)
		{
			for (const TPair<FString, FConfigSection>& Pair : *InstallBundleConfig)
			{
				if (Pair.Key.StartsWith(InstallBundleUtil::GetInstallBundleSectionPrefix()))
				{
					SectionNames.Add(Pair.Key);
				}
			}

			//Skip sorting SectionNames if we have already loaded everything as we will not be applying any regex anyway
			if (!bDidLoadAllFilesFromMetadata)
			{
				// Bundle regex need to be applied in order
				SectionNames.StableSort([InstallBundleConfig](const FString& SectionA, const FString& SectionB) -> bool
				{
					int32 BundleAOrder = INT_MAX;
					int32 BundleBOrder = INT_MAX;

					if (!InstallBundleConfig->GetInt(*SectionA, TEXT("Order"), BundleAOrder))
					{
						LOG_SOURCE_BULK(Warning, TEXT("Bundle Section %s doesn't have an order"), *SectionA);
					}

					if (!InstallBundleConfig->GetInt(*SectionB, TEXT("Order"), BundleBOrder))
					{
						LOG_SOURCE_BULK(Warning, TEXT("Bundle Section %s doesn't have an order"), *SectionB);
					}

					return BundleAOrder < BundleBOrder;
				});
			}

			//Ensure all known sections are appended to the BulkBuildBundles Map even if they have no files
			for (const FString& Section : SectionNames)
			{
				const FString BundleName = Section.RightChop(InstallBundleUtil::GetInstallBundleSectionPrefix().Len());
				BulkBuildBundles.FindOrAdd(FName(*BundleName));
			}

			TSharedPtr<IInstallBundleManager> BundleManger = IInstallBundleManager::GetPlatformInstallBundleManager();

			//If we had files remaining to manually parse, now sort the remaining files into bundles
			if (!bDidLoadAllFilesFromMetadata)
			{
				TRACE_CPUPROFILER_EVENT_SCOPE(FInstallBundleSourceBulk_MatchFiles);
				
				struct FSectionParallelData
				{
					FString BundleName;
					TBitArray<> MatchedFiles;
					bool bAnyMatches;
				};
				TArray<FSectionParallelData> ParallelData;
				ParallelData.AddDefaulted(SectionNames.Num());
				
				// If the file was matched by a bundle with higher priority we can skip it
				// Used as an optimization - don't trust the value in here!
				TArray<std::atomic<int32>> MatchedBy;
				MatchedBy.AddDefaulted(FoundFiles->Num());
				for (int32 I = 0; I < MatchedBy.Num(); ++I)
				{
					MatchedBy[I] = INT32_MAX;
				}
				
				EParallelForFlags ParallelFlags = EParallelForFlags::Unbalanced;
				
				bool bParallelRegexMatching = true;
				
				// Search configs if we should MT
				{
					GConfig->GetBool(TEXT("InstallBundleSource.Bulk.MiscSettings"), TEXT("bParallelRegexMatching"), bParallelRegexMatching, GInstallBundleIni);
	
					if (const FString* ParallelMatchingVar = FPlatformMisc::GetConfigRulesVariable(TEXT("bParallelRegexMatching")))
					{
						bParallelRegexMatching = FCString::ToBool(**ParallelMatchingVar);
					}
					
#if !UE_BUILD_SHIPPING
					if (FParse::Param(FCommandLine::Get(), TEXT("NoParallelBundleRegexMatching")))
					{
						bParallelRegexMatching = false;
					}
#endif // !UE_BUILD_SHIPPING
				}
				
				if (!bParallelRegexMatching)
				{
					ParallelFlags |= EParallelForFlags::ForceSingleThread;
				}
				
				// Match found files for each section in parallel
				ParallelFor(SectionNames.Num(), [&](int32 SectionIndex)
				{
					const FString& Section = SectionNames[SectionIndex];
					FString BundleName = Section.RightChop(InstallBundleUtil::GetInstallBundleSectionPrefix().Len());
					FSectionParallelData& Data = ParallelData[SectionIndex];
					Data.BundleName = BundleName;
					Data.MatchedFiles.Init(false, FoundFiles->Num());
					Data.bAnyMatches = false;

					TArray<FString> StrSearchRegexPatterns;
					InstallBundleConfig->GetArray(*Section, TEXT("FileRegex"), StrSearchRegexPatterns);

					if (StrSearchRegexPatterns.IsEmpty())
					{
						return;
					}

					TArray<FString> BundleSubsets;
					InstallBundleConfig->GetArray(*Section, TEXT("Subsets"), BundleSubsets);
					const int32 iSubset = BundleSubsets.IsEmpty() ? INDEX_NONE : BundleManger->SelectBundleSubset(BundleName, BundleSubsets);

					TArray<FString> StrSubsetSearchRegexPatterns;
					if (BundleSubsets.IsValidIndex(iSubset))
					{
						const FString SubsetSection = InstallBundleUtil::GetInstallBundleSubsetSectionPrefix() + BundleSubsets[iSubset];
						InstallBundleConfig->GetArray(*SubsetSection, TEXT("FileRegex"), StrSubsetSearchRegexPatterns);
					}
					
					TArray<FRegexPattern> SearchRegexPatterns;
					SearchRegexPatterns.Reserve(StrSearchRegexPatterns.Num());
					for (const FString& Str : StrSearchRegexPatterns)
					{
						SearchRegexPatterns.Emplace(Str, ERegexPatternFlags::CaseInsensitive);
					}

					TArray<FRegexPattern> SubsetSearchRegexPatterns;
					SubsetSearchRegexPatterns.Reserve(StrSubsetSearchRegexPatterns.Num());
					for (const FString& Str : StrSubsetSearchRegexPatterns)
					{
						SubsetSearchRegexPatterns.Emplace(Str, ERegexPatternFlags::CaseInsensitive);
					}
					
					for (int32 I = 0; I < FoundFiles->Num(); ++I)
					{
						// Skip if file was already matched by a bundle with higher order
						if (MatchedBy[I].load(std::memory_order_relaxed) <= SectionIndex)
						{
							continue;
						}
						
						FString& File = (*FoundFiles)[I];

						bool bMatches = false;
						for (const FRegexPattern& Pattern : SearchRegexPatterns)
						{
							if (FRegexMatcher(Pattern, File).FindNext())
							{
								bMatches = true;
								break;
							}
						}

						if (!bMatches)
						{
							continue;
						}

						LOG_SOURCE_BULK(Verbose, TEXT("Adding %s to Bundle %s"), *File, *BundleName);

						bMatches = SubsetSearchRegexPatterns.IsEmpty();
						for (const FRegexPattern& Pattern : SubsetSearchRegexPatterns)
						{
							if (FRegexMatcher(Pattern, File).FindNext())
							{
								bMatches = true;
								break;
							}
						}
						
						// This should really be a locked min - which we don't have.
						// Would have to be a CAS loop but from my testing that was slower than doing this.
						// This will give us some false positives but it's "good enough" and saves quite a bit of time.
						if (MatchedBy[I].load(std::memory_order_relaxed) > SectionIndex)
						{
							MatchedBy[I].store(SectionIndex, std::memory_order_relaxed);
						}

						Data.MatchedFiles[I] = bMatches;
						Data.bAnyMatches |= bMatches;
						if (!bMatches)
						{
							LOG_SOURCE_BULK(Verbose, TEXT("Excluded %s from Bundle %s with subset %s"), *File, *BundleName, *BundleSubsets[iSubset]);
						}
					}
				}, ParallelFlags);
				
				// We want files to be added to the BulkBuildBundles in the order that they are defined in SectionNames
				TBitArray<> FoundFilesBits;
				FoundFilesBits.Init(false, FoundFiles->Num());
				for (int32 SectionIndex = 0; SectionIndex < SectionNames.Num(); ++SectionIndex)
				{
					FSectionParallelData& Data = ParallelData[SectionIndex];
					if (Data.bAnyMatches)
					{
						TArray<FString>& BundleFileList = BulkBuildBundles.FindChecked(FName(*Data.BundleName));
						for (int32 I = 0; I < FoundFiles->Num(); ++I)
						{
							FString& File = (*FoundFiles)[I];
							if (Data.MatchedFiles[I] && !FoundFilesBits[I])
							{
								FoundFilesBits[I] = true;
								BundleFileList.AddUnique(MoveTemp(File));
							}
						}
					}
				}
			}
		}

		//See if we should serialize out the manually parsed results
		/*if (!bDidLoadAllFilesFromMetadata)
		{
			bool bShouldSerializeMissingBulkBuildDataIni = false;
			if (!GConfig->GetBool(TEXT("InstallBundleSource.Bulk.MiscSettings"), TEXT("bShouldSerializeMissingBulkBuildDataIni"), bShouldSerializeMissingBulkBuildDataIni, GInstallBundleIni))
			{
				bShouldSerializeMissingBulkBuildDataIni = false;
			}

			if (bShouldSerializeMissingBulkBuildDataIni)
			{
				SerializeBulkBuildBundleMetadata(BulkBuildBundles);
			}
		}*/

		LOG_SOURCE_BULK(Display, TEXT("Finished Making Bundles for Bulk Build"));
		InitStepResult = EAsyncInitStepResult::Done;
	});
}

bool FInstallBundleSourceBulk::TryLoadBulkBuildBundleMetadata(TArray<FString>& InOutFileList, TMap<FName, TArray<FString>>& InOutBulkBuildBundles)
{
	FBulkBuildBundleMapJsonInfo LoadedBuildInfo;

	//Always prioritize loading from the LocalCache
	if (!LoadedBuildInfo.LoadFromFile(FBulkBuildBundleMapJsonInfo::GetBulkBuildBundleInfoLocalCachedPath()))
	{
		//If there is no local cache look for a cooked one
		LoadedBuildInfo.LoadFromFile(FBulkBuildBundleMapJsonInfo::GetBulkBuildBundleInfoCookedPath());
	}

	if (!LoadedBuildInfo.IsEmpty())
	{
		return LoadedBuildInfo.AppendEntriesToBulkBuildBundleMap(InOutFileList, InOutBulkBuildBundles);
	}

	return false;
}

void FInstallBundleSourceBulk::SerializeBulkBuildBundleMetadata(const TMap<FName, TArray<FString>>& BulkBuildBundles)
{
	FBulkBuildBundleMapJsonInfo JsonBulkBuildInfo(BulkBuildBundles);
	if (!JsonBulkBuildInfo.IsEmpty())
	{
		FStringView FilePath = FBulkBuildBundleMapJsonInfo::GetBulkBuildBundleInfoLocalCachedPath();
		const bool bSuccess = JsonBulkBuildInfo.SaveToFile(FilePath);
		LOG_SOURCE_BULK(Display, TEXT("Saving BulkBuildBundle Cache to %.*s . bSuccess:%s"), FilePath.Len(), FilePath.GetData(), *LexToString(bSuccess));
	}
}

EInstallBundleInstallState FInstallBundleSourceBulk::GetBundleInstallState(FName BundleName)
{
	return EInstallBundleInstallState::UpToDate;
}

FInstallBundleSourceType FInstallBundleSourceBulk::GetSourceType() const
{ 
	return FInstallBundleSourceType(TEXT("Bulk")); 
}

FInstallBundleSourceInitInfo FInstallBundleSourceBulk::Init(
	TSharedRef<InstallBundleUtil::FContentRequestStatsMap> InRequestStats,
	TSharedPtr<IAnalyticsProviderET> InAnalyticsProvider,
	TSharedPtr<InstallBundleUtil::PersistentStats::FPersistentStatContainerBase> PersistentStatsContainer)
{
	AnalyticsProvider = MoveTemp(InAnalyticsProvider);

	//Ignoring PersistentStatsContainer as we currently don't care about any stats for this bulk source

	FInstallBundleSourceInitInfo InitInfo;
	return InitInfo;
}

void FInstallBundleSourceBulk::AsyncInit(FInstallBundleSourceInitDelegate Callback)
{
	check(OnInitCompleteCallback.IsBound() == false);
	OnInitCompleteCallback = MoveTemp(Callback);

	if (InitState == EInstallBundleManagerInitState::Failed)
	{
		InitState = EInstallBundleManagerInitState::NotInitialized;
		bRetryInit = true;
	}
}

void FInstallBundleSourceBulk::AsyncInit_QueryBundleInfo(FInstallBundleSourceQueryBundleInfoDelegate Callback)
{
	check(InitState == EInstallBundleManagerInitState::Succeeded);

	FInstallBundleSourceBundleInfoQueryResult ResultInfo;

	const FConfigFile* InstallBundleConfig = GConfig->FindConfigFile(GInstallBundleIni);
	if (InstallBundleConfig)
	{
		for (const TPair<FString, FConfigSection>& Pair : *InstallBundleConfig)
		{
			const FString& Section = Pair.Key;
			FInstallBundleSourcePersistentBundleInfo BundleInfo;
			if (!InstallBundleManagerUtil::LoadBundleSourceBundleInfoFromConfig(GetSourceType(), *InstallBundleConfig, Section, BundleInfo))
			{
				continue;
			}

			BundleInfo.BundleContentState = GetBundleInstallState(BundleInfo.BundleName);

			if (TArray<FString>* FileList = BulkBuildBundles.Find(BundleInfo.BundleName))
			{
				BundleInfo.bContainsIoStoreOnDemandToc = Algo::AnyOf(
					*FileList, [](FStringView File) { return File.EndsWith(TEXTVIEW(".uondemandtoc")); });
			}

			FName BundleName = BundleInfo.BundleName;
			ResultInfo.SourceBundleInfoMap.Add(BundleName, MoveTemp(BundleInfo));
		}
	}

	Callback.ExecuteIfBound(AsShared(), MoveTemp(ResultInfo));
}

EInstallBundleManagerInitState FInstallBundleSourceBulk::GetInitState() const
{
	return InitState;
}

FString FInstallBundleSourceBulk::GetContentVersion() const
{
	return InstallBundleUtil::GetAppVersion();
}

TSet<FName> FInstallBundleSourceBulk::GetBundleDependencies(FName InBundleName, TSet<FName>* SkippedUnknownBundles /*= nullptr*/) const
{
	return InstallBundleManagerUtil::GetBundleDependenciesFromConfig(InBundleName, SkippedUnknownBundles);
}

void FInstallBundleSourceBulk::GetContentState(TArrayView<const FName> BundleNames, EInstallBundleGetContentStateFlags Flags, FInstallBundleGetContentStateDelegate Callback)
{
	FInstallBundleCombinedContentState State;
	State.CurrentVersion.Add(GetSourceType(), InstallBundleUtil::GetAppVersion());
	for (const FName& BundleName : BundleNames)
	{
		if(!BulkBuildBundles.Contains(BundleName))
			continue;

		LOG_SOURCE_BULK(Verbose, TEXT("Requesting Content State for %s"), *BundleName.ToString());

		FInstallBundleContentState& IndividualBundleState = State.IndividualBundleStates.Add(BundleName);
		IndividualBundleState.State = GetBundleInstallState(BundleName);
		IndividualBundleState.Version.Add(GetSourceType(), InstallBundleUtil::GetAppVersion());
	}

	for (TPair<FName, FInstallBundleContentState>& Pair : State.IndividualBundleStates)
	{
		Pair.Value.Weight = 1.0f / State.IndividualBundleStates.Num();
	}

	Callback.ExecuteIfBound(MoveTemp(State));
}

void FInstallBundleSourceBulk::RequestUpdateContent(FRequestUpdateContentBundleContext Context)
{
	FNameBuilder BundleNameBuilder(Context.BundleName);

	LOG_SOURCE_BULK_OVERRIDE(Context.LogVerbosityOverride, Display, TEXT("Requesting Bundle %s"), BundleNameBuilder.ToString());

	InstallBundleUtil::FConfigMountOptions OnDemandMountOptions;
	InstallBundleUtil::GetMountOptionsFromConfig(BundleNameBuilder, OnDemandMountOptions);

	FInstallBundleSourceUpdateContentResultInfo ResultInfo;
	ResultInfo.BundleName = Context.BundleName;
	ResultInfo.Result = EInstallBundleResult::OK;
	TArray<FString>* BundleFileList = BulkBuildBundles.Find(Context.BundleName);
	if (BundleFileList)
	{
		ResultInfo.ContentPaths = *BundleFileList;
	}

	FilterContentBundlePaths(Context, ResultInfo.ContentPaths);

	if (OnDemandMountOptions.bWithSoftReferences)
	{
		ResultInfo.MountOptions.MountFlags |= FPakMountOptions::EMountFlags::WithSoftReferences;
	}
	
	ResultInfo.ProjectName = FApp::GetProjectName();
	Context.CompleteCallback.ExecuteIfBound(AsShared(), MoveTemp(ResultInfo));
}

void FInstallBundleSourceBulk::SetErrorSimulationCommands(const FString& CommandLine)
{
#if INSTALL_BUNDLE_ALLOW_ERROR_SIMULATION
#endif // INSTALL_BUNDLE_ALLOW_ERROR_SIMULATION
}

const TCHAR* LexToString(FInstallBundleSourceBulk::EAsyncInitStep Val)
{
	static const TCHAR* Strings[] =
	{
		TEXT("InstallBundleSourceBulk:None"),
		TEXT("InstallBundleSourceBulk:MakeBundlesForBulkBuild"),
		TEXT("InstallBundleSourceBulk:Finishing"),
	};
	static_assert(InstallBundleUtil::CastToUnderlying(FInstallBundleSourceBulk::EAsyncInitStep::Count) == UE_ARRAY_COUNT(Strings), "");

	return Strings[InstallBundleUtil::CastToUnderlying(Val)];
}
