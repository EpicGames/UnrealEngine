// Copyright Epic Games, Inc. All Rights Reserved.

#include "InstallBundleSourcePlatformBase.h"


#if WITH_PLATFORM_INSTALL_BUNDLE_SOURCE || WITH_PLATFORM_DLC_INSTALL_BUNDLE_SOURCE
#include "DefaultInstallBundleManagerPrivate.h"
#include "InstallBundleManagerUtil.h"

#include "Misc/ConfigCacheIni.h"
#include "Stats/Stats.h"

#define LOG_SOURCE_PLATFORMBASE(Verbosity, Format, ...) LOG_INSTALL_BUNDLE_MAN(Verbosity, TEXT("InstallBundleSourcePlatformBase: ") Format, ##__VA_ARGS__)

#define LOG_SOURCE_PLATFORMBASE_OVERRIDE(VerbosityOverride, Verbosity, Format, ...) LOG_INSTALL_BUNDLE_MAN_OVERRIDE(VerbosityOverride, Verbosity, TEXT("InstallBundleSourcePlatformBase: ") Format, ##__VA_ARGS__)


FInstallBundleSourcePlatformBase::FInstallBundleSourcePlatformBase()
{
}

FInstallBundleSourcePlatformBase::~FInstallBundleSourcePlatformBase()
{
}


FInstallBundleSourceType FInstallBundleSourcePlatformBase::GetSourceType() const
{
	return FInstallBundleSourceType(TEXT("Platform"));
}

FInstallBundleSourceInitInfo FInstallBundleSourcePlatformBase::Init(
	TSharedRef<InstallBundleUtil::FContentRequestStatsMap> InRequestStats, 
	TSharedPtr<IAnalyticsProviderET> InAnalyticsProvider,
	TSharedPtr<InstallBundleUtil::PersistentStats::FPersistentStatContainerBase> InPersistentStatsContainer)
{
	CSV_SCOPED_TIMING_STAT(InstallBundleManager, FInstallBundleSourcePlatformBase_Init);

	RequestStats = MoveTemp(InRequestStats);
	AnalyticsProvider = MoveTemp(InAnalyticsProvider);
	PersistentStatsContainer = MoveTemp(InPersistentStatsContainer);

	FInstallBundleSourceInitInfo InitInfo;

	const FConfigFile* InstallBundleConfig = GConfig->FindConfigFile(GInstallBundleIni);
	if (!InstallBundleConfig)
	{
		InitState = EInstallBundleManagerInitState::Failed;
		InitInfo.Result = EInstallBundleManagerInitResult::ConfigurationError;

		LOG_SOURCE_PLATFORMBASE(Error, TEXT("Initialization Failed - %s"), LexToString(InitInfo.Result));
		return InitInfo;
	}

	BundleRegexList = InstallBundleUtil::LoadBundleRegexFromConfig(*InstallBundleConfig, InstallBundleUtil::IsPlatformInstallBundlePredicate);
	BundleSubsetRegexList = InstallBundleUtil::LoadBundleSubsetRegexFromConfig(*InstallBundleConfig);

	return InitInfo;
}

void FInstallBundleSourcePlatformBase::AsyncInit_QueryBundleInfo(FInstallBundleSourceQueryBundleInfoDelegate Callback)
{
	CSV_SCOPED_TIMING_STAT(InstallBundleManager, FInstallBundleSourcePlatformBase_AsyncInit_QueryBundleInfo);

	check(InitState == EInstallBundleManagerInitState::Succeeded);

	FInstallBundleSourceBundleInfoQueryResult ResultInfo;

	const FConfigFile* InstallBundleConfig = GConfig->FindConfigFile(GInstallBundleIni);
	if (InstallBundleConfig)
	{
		for (const TPair<FString, FConfigSection>& Pair : *InstallBundleConfig)
		{
			const FString& Section = Pair.Key;
			FInstallBundleSourcePersistentBundleInfo SourceBundleInfo;
			if (!InstallBundleManagerUtil::LoadBundleSourceBundleInfoFromConfig(GetSourceType(), *InstallBundleConfig, Section, SourceBundleInfo))
				continue;

			if (!QueryPersistentBundleInfo(SourceBundleInfo))
				continue;

			FName BundleName = SourceBundleInfo.BundleName;
			ResultInfo.SourceBundleInfoMap.Add(BundleName, MoveTemp(SourceBundleInfo));
		}
	}

	Callback.ExecuteIfBound(AsShared(), MoveTemp(ResultInfo));
}

EInstallBundleManagerInitState FInstallBundleSourcePlatformBase::GetInitState() const
{
	return InitState;
}

FString FInstallBundleSourcePlatformBase::GetContentVersion() const
{
	return InstallBundleUtil::GetAppVersion();
}

TSet<FName> FInstallBundleSourcePlatformBase::GetBundleDependencies(FName InBundleName, TSet<FName>* SkippedUnknownBundles /*= nullptr*/) const
{
	return InstallBundleManagerUtil::GetBundleDependenciesFromConfig(InBundleName, SkippedUnknownBundles);
}

#endif //WITH_PLATFORM_INSTALL_BUNDLE_SOURCE || WITH_PLATFORM_DLC_INSTALL_BUNDLE_SOURCE
