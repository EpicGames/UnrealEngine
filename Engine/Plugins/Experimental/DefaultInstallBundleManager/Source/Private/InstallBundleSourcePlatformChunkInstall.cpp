// Copyright Epic Games, Inc. All Rights Reserved.

#include "InstallBundleSourcePlatformChunkInstall.h"
#if WITH_PLATFORM_INSTALL_BUNDLE_SOURCE
#include "DefaultInstallBundleManagerPrivate.h"

#include "InstallBundleManagerUtil.h"
#include "InstallBundleManagerInterface.h"

#include "Misc/CommandLine.h"
#include "Misc/ConfigCacheIni.h"
#include "Algo/AnyOf.h"
#include "Algo/Find.h"
#include "Containers/Ticker.h"
#include "IPlatformFilePak.h"
#include "Stats/Stats.h"

#define LOG_SOURCE_CHUNKINSTALL(Verbosity, Format, ...) LOG_INSTALL_BUNDLE_MAN(Verbosity, TEXT("InstallBundleSourcePlatformChunkInstall: ") Format, ##__VA_ARGS__)

#define LOG_SOURCE_CHUNKINSTALL_OVERRIDE(VerbosityOverride, Verbosity, Format, ...) LOG_INSTALL_BUNDLE_MAN_OVERRIDE(VerbosityOverride, Verbosity, TEXT("InstallBundleSourcePlatformChunkInstall: ") Format, ##__VA_ARGS__)



FInstallBundleSourcePlatformChunkInstall::FInstallBundleSourcePlatformChunkInstall(IPlatformChunkInstall* InPlatformChunkInstall)
	: PlatformChunkInstall(InPlatformChunkInstall)
{
	TickHandle = FTSTicker::GetCoreTicker().AddTicker(FTickerDelegate::CreateRaw(this, &FInstallBundleSourcePlatformChunkInstall::Tick));

	NamedChunkInstallDelegateHandle = PlatformChunkInstall->AddNamedChunkCompleteDelegate( FPlatformNamedChunkCompleteDelegate::CreateRaw(this, &FInstallBundleSourcePlatformChunkInstall::OnNamedChunkInstall));

	InPlatformChunkInstall->SetAutoPakMountingEnabled(false);
}


FInstallBundleSourcePlatformChunkInstall::~FInstallBundleSourcePlatformChunkInstall()
{
	PlatformChunkInstall->RemoveNamedChunkCompleteDelegate(NamedChunkInstallDelegateHandle);
	NamedChunkInstallDelegateHandle.Reset();

	FTSTicker::RemoveTicker(TickHandle);
	TickHandle.Reset();

	// Cleanup any Async tasks
	InstallBundleUtil::CleanupInstallBundleAsyncIOTasks(GeneralAsyncTasks);
}


bool FInstallBundleSourcePlatformChunkInstall::Tick(float dt)
{
	QUICK_SCOPE_CYCLE_COUNTER(STAT_FInstallBundleSourcePlatformChunkInstall_Tick);

	CSV_SCOPED_TIMING_STAT(InstallBundleManager, FInstallBundleSourcePlatformChunkInstall_Tick);

	InstallBundleUtil::FinishInstallBundleAsyncIOTasks(GeneralAsyncTasks);

	// remove finished requests
	ContentReleaseRequests.RemoveAll([](const FContentReleaseRequestRef& Request) { return !Request->bInProgress; });
	bool bHasRemovedRequests = ContentRequests.RemoveAll([](const FContentRequestRef& Request) { return !Request->bInProgress; }) > 0;
	bChunkOrderDirty |= bHasRemovedRequests;

	// make sure we're installing the highest priority chunks
	TickUpdateChunkOrder();

	return true;
}


void FInstallBundleSourcePlatformChunkInstall::TickUpdateChunkOrder()
{
	CSV_SCOPED_TIMING_STAT(InstallBundleManager, FInstallBundleSourcePlatformChunkInstall_TickUpdateChunkOrder);

	if (bChunkOrderDirty && !bIsUpdatingChunkOrder)
	{
		bChunkOrderDirty = false; // immediately clear dirty flag as we may get dirty while updating
		bIsUpdatingChunkOrder = true;

		// find the highest priority group of currently-installing bundles
		EInstallBundlePriority HighestPriority = EInstallBundlePriority::Count;
		TArray<FName> HighestPriorityNamedChunks;

		for (const FContentRequestRef& ContentRequest : ContentRequests)
		{
			const FBundleInfo* BundleInfo = BundleInfoMap.Find(ContentRequest->BundleName);
			if (BundleInfo != nullptr)
			{
				if (BundleInfo->Priority < HighestPriority) // NB. lower value = more important!
				{
					HighestPriorityNamedChunks.Reset();
					HighestPriority = BundleInfo->Priority;
				}
				if (BundleInfo->Priority == HighestPriority)
				{
					HighestPriorityNamedChunks.Add(BundleInfo->NamedChunk);
				}
			}
		}

		// update the feature priority asyncronously
		if (HighestPriorityNamedChunks.Num() > 0)
		{
			InstallBundleUtil::StartInstallBundleAsyncIOTask(GeneralAsyncTasks, 
				[this,PriorityNamedChunks = MoveTemp(HighestPriorityNamedChunks)]()
				{
					for (FName NamedChunk : PriorityNamedChunks)
					{
						PlatformChunkInstall->PrioritizeNamedChunk( NamedChunk, EChunkPriority::High );
					}
				},
				[this]()
				{
					bIsUpdatingChunkOrder = false;
				}
			);
		}
		else
		{
			// nothing to update
			bIsUpdatingChunkOrder = false;
		}
	}
}


FInstallBundleSourceInitInfo FInstallBundleSourcePlatformChunkInstall::Init(TSharedRef<InstallBundleUtil::FContentRequestStatsMap> InRequestStats, TSharedPtr<IAnalyticsProviderET> InAnalyticsProvider, TSharedPtr<InstallBundleUtil::PersistentStats::FPersistentStatContainerBase> InPersistentStatsContainer)
{
	CSV_SCOPED_TIMING_STAT(InstallBundleManager, FInstallBundleSourcePlatformChunkInstall_Init);
	FInstallBundleSourceInitInfo InitInfo;

	if (!PlatformChunkInstall->SupportsBundleSource() || !PlatformChunkInstall->IsAvailable())
	{
		LOG_SOURCE_CHUNKINSTALL(Display, TEXT("Platform chunk installer doesn't support bundles or this is not a packaged build, attempting to fallback to next bundle source."));

		InitState = EInstallBundleManagerInitState::Failed;

		InitInfo.Result = EInstallBundleManagerInitResult::BuildMetaDataNotFound;
		InitInfo.bShouldUseFallbackSource = true;
		return InitInfo;
	}

	InitInfo = FInstallBundleSourcePlatformBase::Init(InRequestStats, InAnalyticsProvider, InPersistentStatsContainer);
	return InitInfo;
}


void FInstallBundleSourcePlatformChunkInstall::AsyncInit(FInstallBundleSourceInitDelegate Callback)
{
	CSV_SCOPED_TIMING_STAT(InstallBundleManager, FInstallBundleSourcePlatformChunkInstall_AsyncInit);

	// Handle retrying init if recoverable
	if (InitState == EInstallBundleManagerInitState::Failed)
	{
		LOG_SOURCE_CHUNKINSTALL(Warning, TEXT("Retrying initialization"));
		InitState = EInstallBundleManagerInitState::NotInitialized;
	}

	struct AsyncInitContext
	{
		EInstallBundleManagerInitResult InitResult = EInstallBundleManagerInitResult::OK;

		TSet<FName> NamedChunks;
		TMap<FName, FBundleInfo> BundleInfoMap;
		TMap<FName, TArray<FString>> BundleToSubsetMap;
		TArray<TPair<FString, TArray<FRegexPattern>>> BundleRegexList;
		TArray<TPair<FString, TArray<FRegexPattern>>> BundleSubsetRegexList;
		IPlatformChunkInstall* PlatformChunkInstall;
		TSharedPtr<IInstallBundleManager> BundleManager;
	};


	// create and initialize the async init context
	TSharedPtr<AsyncInitContext, ESPMode::ThreadSafe> Context = MakeShared<AsyncInitContext, ESPMode::ThreadSafe>();
	Context->BundleRegexList = MoveTemp(BundleRegexList);
	Context->BundleSubsetRegexList = MoveTemp(BundleSubsetRegexList);
	Context->PlatformChunkInstall = PlatformChunkInstall;
	Context->BundleManager = IInstallBundleManager::GetPlatformInstallBundleManager();
	

	auto AsyncInit = [Context]() mutable
	{
		// make sure there's a config
		const FConfigFile* InstallBundleConfig = GConfig->FindConfigFile(GInstallBundleIni);
		if (!InstallBundleConfig)
		{
			Context->InitResult = EInstallBundleManagerInitResult::ConfigurationError;
			return;
		}

		// read the default chunk name. this is used for bundles that have no platform data configured
		FString DefaultPlatformChunkName;
		GConfig->GetString(TEXT("InstallBundleSource.Platform.MiscSettings"), TEXT("DefaultPlatformChunkName"), DefaultPlatformChunkName, GInstallBundleIni);

		// cache all named chunks
		Context->NamedChunks.Append( Context->PlatformChunkInstall->GetNamedChunksByType(ENamedChunkType::OnDemand) );
		Context->NamedChunks.Append( Context->PlatformChunkInstall->GetNamedChunksByType(ENamedChunkType::Language) );
		LOG_SOURCE_CHUNKINSTALL(Display, TEXT("Platform chunk install has %d named chunks"), Context->NamedChunks.Num() );

		// create bundles & chunk data
		TSet<FName> KnownNamedChunks;
		TMap<FName, TArray<FString>> NamedChunkFilePaths;
		FString RootDir = FPaths::RootDir();
		bool bHasAbsoluteRootDir = !FPaths::IsRelative(RootDir);

		for (const TPair<FString, FConfigSection>& Pair : *InstallBundleConfig)
		{
			// make sure this is a suitable platform bundle
			const FString& Section = Pair.Key;
			if (!Section.StartsWith(InstallBundleUtil::GetInstallBundleSectionPrefix()))
			{
				continue;
			}
			
			// read the PlatformChunkID. Note this is only used for checking disabled or defaulted bundles (such as Game Feature Plugins)
			// it isn't actually used by this bundle source. At some point a PlatformChunkName-based equivalent will be added instead
			int32 PlatformChunkID = 0;
			InstallBundleConfig->GetInt(*Section, TEXT("PlatformChunkID"), PlatformChunkID);
			if (PlatformChunkID < 0)
			{
				continue;
			}
			
			FString PlatformChunkName;
			InstallBundleConfig->GetString(*Section, TEXT("PlatformChunkName"), PlatformChunkName);
			if (PlatformChunkID == 0 && PlatformChunkName.IsEmpty())
			{
				// This bundle has no specific PlatformChunkName and the PlatformChunkID is either 0 ("default") or unspecified, so use the name of the default chunk name.
				// The code below will assume the chunk name is the same as the bundle name if it's not been specified.
				PlatformChunkName = DefaultPlatformChunkName;
			}

			// create bundle info
			FName BundleName( *Section.RightChop(InstallBundleUtil::GetInstallBundleSectionPrefix().Len()));
			FBundleInfo& BundleInfo = Context->BundleInfoMap.Add(BundleName);
			BundleInfo.NamedChunk = PlatformChunkName.IsEmpty() ? BundleName : FName(*PlatformChunkName);
			BundleInfo.Priority = EInstallBundlePriority::Normal;

			FString PriorityString;
			InstallBundleConfig->GetString(*Section, TEXT("Priority"), PriorityString);
			LexTryParseString(BundleInfo.Priority, *PriorityString);

			TArray<FString> BundleSubsets;
			InstallBundleConfig->GetArray(*Section, TEXT("Subsets"), BundleSubsets);
			if (!BundleSubsets.IsEmpty())
			{
				Context->BundleToSubsetMap.Add(BundleName, MoveTemp(BundleSubsets));
			}

			if (KnownNamedChunks.Contains(BundleInfo.NamedChunk))
			{
				continue;
			}
			KnownNamedChunks.Add(BundleInfo.NamedChunk);

			if (Context->NamedChunks.Contains(BundleInfo.NamedChunk))
			{
				// collect the paths for this bundle
				TArray<FString> FilesInChunk;
				if (Context->PlatformChunkInstall->GetPakFilesInNamedChunk(BundleInfo.NamedChunk, FilesInChunk))
				{
					for (const FString& FilePath : FilesInChunk)
					{
						FString ChunkFilePath = FilePath;
						if (bHasAbsoluteRootDir)
						{
							ChunkFilePath = TEXT("../../../") + ChunkFilePath;
						}
						FPaths::NormalizeFilename(ChunkFilePath);
						NamedChunkFilePaths.FindOrAdd(BundleInfo.NamedChunk).Add(ChunkFilePath);
					}
				}
			}

			LOG_SOURCE_CHUNKINSTALL(Display, TEXT("Initialized chunk %s (%d) for bundle %s %s"), *PlatformChunkName, PlatformChunkID, *BundleName.ToString(), (Context->NamedChunks.Contains(BundleInfo.NamedChunk)) ? TEXT("") : TEXT("...not named chunk!") );
		}

		// collect pak search directories
		TArray<FString> PakFolders;
		FPakPlatformFile::GetPakFolders(FCommandLine::Get(), PakFolders);
		
		TArray<FString> PakSearchDirs;
		PakSearchDirs.Empty(PakFolders.Num());
		for (FString& PakFolder : PakFolders)
		{
			if (bHasAbsoluteRootDir && FPaths::IsRelative(PakFolder))
			{
				PakSearchDirs.Add(MoveTemp(PakFolder));
			}
			else if (PakFolder.StartsWith(RootDir))
			{
				verify(FPaths::MakePathRelativeTo(PakSearchDirs.Add_GetRef(MoveTemp(PakFolder)), *RootDir));
			}
		}

		// More than one bundle can map to the same chunk, so map chunk files to the correct bundles
		for (TPair<FName, TArray<FString>>& FilePathPair : NamedChunkFilePaths)
		{
			for (FString& Path : FilePathPair.Value)
			{
				if (!Algo::AnyOf(PakSearchDirs, [&Path](const FString& Dir) { return Path.StartsWith(Dir); }))
				{
					// Only want to mount content in PakSearchDirs and don't mount
					// any other content that may be in the package.
					continue;
				}
		
				FString BundleName;
				if (InstallBundleUtil::MatchBundleRegex(Context->BundleRegexList, Path, BundleName))
				{
					FName BundleFName(*BundleName);
					FBundleInfo& BundleInfo = Context->BundleInfoMap.FindChecked(BundleFName);
					TArray<FString>* BundleSubsets = Context->BundleToSubsetMap.Find(BundleFName);

					const int32 iSubset = (!BundleSubsets) ? INDEX_NONE : Context->BundleManager->SelectBundleSubset(BundleName, *BundleSubsets);

					if (!BundleSubsets || !BundleSubsets->IsValidIndex(iSubset) ||
						InstallBundleUtil::MatchBundleSubsetRegex((*BundleSubsets)[iSubset], Context->BundleSubsetRegexList, Path))
					{
						BundleInfo.FilePaths.AddUnique(MoveTemp(Path));
					}
				}
				else
				{
					checkf(false, TEXT("Failed to map chunk file %s to an install bundle"), *Path);
				}
			}
		}
	};

	auto OnAsyncInitComplete = [this, Context, OnInitCompleteCallback = MoveTemp(Callback)]() mutable
	{
		check(IsInGameThread());

		FInstallBundleSourceAsyncInitInfo InitInfo;
		InitInfo.Result = Context->InitResult;

		if (InitInfo.Result == EInstallBundleManagerInitResult::OK)
		{
			// initialization succeeded - store the initialized data
			BundleInfoMap = MoveTemp(Context->BundleInfoMap);
			NamedChunks = MoveTemp(Context->NamedChunks);

			for (const auto& BundleInfo : BundleInfoMap)
			{
				LOG_SOURCE_CHUNKINSTALL(Display, TEXT("%32s -> %-32s (%d file paths)"), *BundleInfo.Key.ToString(), *BundleInfo.Value.NamedChunk.ToString(), BundleInfo.Value.FilePaths.Num());
			}
			LOG_SOURCE_CHUNKINSTALL(Display, TEXT("Initialization completed - %d bundles, %d named chunks"), BundleInfoMap.Num(), NamedChunks.Num() );
			InitState = EInstallBundleManagerInitState::Succeeded;
		}
		else
		{
			LOG_SOURCE_CHUNKINSTALL(Error, TEXT("Initialization Failed - %s"), LexToString(InitInfo.Result));
			InitState = EInstallBundleManagerInitState::Failed;
		}

		LOG_SOURCE_CHUNKINSTALL(Display, TEXT("Fire Init Analytic: %s"), LexToString(InitInfo.Result));
		InstallBundleManagerAnalytics::FireEvent_InitBundleSourcePlatformChunkInstallComplete(AnalyticsProvider.Get(), LexToString(InitInfo.Result));

		OnInitCompleteCallback.Execute(AsShared(), MoveTemp(InitInfo));
	};

	auto OnPlatformChunkInstallComplete = [this, Context, AsyncInit = MoveTemp(AsyncInit), OnAsyncInitComplete = MoveTemp(OnAsyncInitComplete)](bool bSuccess) mutable
	{
		check(IsInGameThread());

		if (bSuccess)
		{
			// next start the async bundle initialize
			LOG_SOURCE_CHUNKINSTALL(Display, TEXT("Platform chunk installer init succeeded") );
			InstallBundleUtil::StartInstallBundleAsyncIOTask(GeneralAsyncTasks, MoveTemp(AsyncInit), MoveTemp(OnAsyncInitComplete));
		}
		else
		{
			LOG_SOURCE_CHUNKINSTALL(Error, TEXT("Initialization Failed - platform chunk installer async init failed"));
			Context->InitResult = EInstallBundleManagerInitResult::ConfigurationError;
			OnAsyncInitComplete();
		}
	};
	
	// firstly, start initializing the platform chunk installer
	PlatformChunkInstall->AsyncInit(OnPlatformChunkInstallComplete);
}


bool FInstallBundleSourcePlatformChunkInstall::QueryPersistentBundleInfo(FInstallBundleSourcePersistentBundleInfo& SourceBundleInfo) const
{
	// lookup bundle info
	const FBundleInfo* BundleInfo = BundleInfoMap.Find(SourceBundleInfo.BundleName);
	if (BundleInfo == nullptr)
	{
		return false;
	}

	SourceBundleInfo.bContainsIoStoreOnDemandToc = Algo::AnyOf(
		BundleInfo->FilePaths, [](FStringView File) { return File.EndsWith(TEXTVIEW(".uondemandtoc")); });

	// assume the startup bundle is installed. It cannot be uninstalled
	if (SourceBundleInfo.bIsStartup)
	{
		SourceBundleInfo.BundleContentState = EInstallBundleInstallState::UpToDate;
		return true;
	}

	// lookup the chunk data
	FChunkInstallationStatusDetail ChunkStatusDetail;
	if (PlatformChunkInstall->GetNamedChunkInstallationStatus(BundleInfo->NamedChunk, ChunkStatusDetail))
	{
		SourceBundleInfo.CurrentInstallSize = ChunkStatusDetail.CurrentInstallSize;
		SourceBundleInfo.FullInstallSize = ChunkStatusDetail.FullInstallSize;
		SourceBundleInfo.BundleContentState = ChunkStatusDetail.bIsInstalled ? EInstallBundleInstallState::UpToDate : EInstallBundleInstallState::NotInstalled;
		return true;
	}

	return true;
}


void FInstallBundleSourcePlatformChunkInstall::GetContentState(TArrayView<const FName> BundleNames, EInstallBundleGetContentStateFlags Flags, FInstallBundleGetContentStateDelegate Callback)
{
	CSV_SCOPED_TIMING_STAT(InstallBundleManager, FInstallBundleSourcePlatformChunkInstall_GetContentState);

	FInstallBundleCombinedContentState State;
	State.CurrentVersion.Add(GetSourceType(), InstallBundleUtil::GetAppVersion());

	// capture the install state and version for all bundles, and record the remaining download size
	TMap<FName,uint64> BundleToRemaingDownloadSize;
	for (const FName& BundleName : BundleNames)
	{
		const FBundleInfo* BundleInfo = BundleInfoMap.Find(BundleName);
		if (BundleInfo != nullptr)
		{
			FChunkInstallationStatusDetail ChunkStatusDetail;
			bool bHasChunkStatus = PlatformChunkInstall->GetNamedChunkInstallationStatus( BundleInfo->NamedChunk, ChunkStatusDetail );

			FInstallBundleContentState& IndividualBundleState = State.IndividualBundleStates.Add(BundleName);
			if (bHasChunkStatus && ChunkStatusDetail.bIsInstalled)
			{
				IndividualBundleState.State = EInstallBundleInstallState::UpToDate;
				IndividualBundleState.Version.Add(GetSourceType(), InstallBundleUtil::GetAppVersion());
			}
			else
			{
				// named chunk is either not installed, or it's not a valid named chunk (also meaning it isn't installed)... 
				// an invalid named chunk likely means there is an entry for it in the bundle ini, but the platform chunk it refers to does not exist
				IndividualBundleState.State = EInstallBundleInstallState::NotInstalled;
				IndividualBundleState.Version.Add(GetSourceType(), TEXT(""));
			}

			uint64 RemainingDownloadSize = 0;
			if (bHasChunkStatus)
			{
				check( ChunkStatusDetail.CurrentInstallSize <= ChunkStatusDetail.FullInstallSize);
				RemainingDownloadSize = (ChunkStatusDetail.FullInstallSize - ChunkStatusDetail.CurrentInstallSize);

				State.ContentSize.DownloadSize += RemainingDownloadSize;
				State.ContentSize.CurrentSizeOnDiskOtherDirs += ChunkStatusDetail.CurrentInstallSize;
				State.ContentSize.SpaceRequiredForInstallOtherDirs += RemainingDownloadSize;
			}
			BundleToRemaingDownloadSize.Add(BundleName, RemainingDownloadSize);
		}
	}

	// compute the download weight for all of the known bundles - higher weight is a bigger download
	for (TTuple<FName,FInstallBundleContentState>& BundleStatePair : State.IndividualBundleStates)
	{
		if (State.ContentSize.DownloadSize == 0)
		{
			BundleStatePair.Value.Weight = 1.0f / BundleNames.Num();
			continue;
		}

		double Weight = (double)BundleToRemaingDownloadSize[BundleStatePair.Key] / (double)State.ContentSize.DownloadSize;
		BundleStatePair.Value.Weight = FMath::Max(Weight, InstallBundleUtil::MinimumBundleWeight); // this Max() does mean the total weight will be > 1.0 but all other bundle sources do it this way too
	}

	Callback.ExecuteIfBound(MoveTemp(State));
}


EInstallBundleSourceBundleSkipReason FInstallBundleSourcePlatformChunkInstall::GetBundleSkipReason(FName BundleName) const
{
	EInstallBundleSourceBundleSkipReason SkipReason = EInstallBundleSourceBundleSkipReason::None;

	const FBundleInfo* BundleInfo = BundleInfoMap.Find(BundleName);
	if (BundleInfo == nullptr)
	{
		LOG_SOURCE_CHUNKINSTALL(Verbose, TEXT("Skipping Bundle %s - no equivalent platform named chunk - allowing anyway"), *BundleName.ToString());
		return SkipReason;
	}

	bool SkipLanguageBundlesIfNotForCurrentLocale = true;
	if (!GConfig->GetBool(TEXT("InstallBundleSource.Platform.MiscSettings"), TEXT("SkipLanguageBundlesIfNotForCurrentLocale"), SkipLanguageBundlesIfNotForCurrentLocale, GInstallBundleIni))
	{
		SkipLanguageBundlesIfNotForCurrentLocale = true;
	}
	if (SkipLanguageBundlesIfNotForCurrentLocale && !PlatformChunkInstall->IsNamedChunkForCurrentLocale(BundleInfo->NamedChunk))
	{
		LOG_SOURCE_CHUNKINSTALL(Verbose, TEXT("Skipping Bundle %s - Locale not current"), *BundleName.ToString());
		SkipReason |= EInstallBundleSourceBundleSkipReason::LanguageNotCurrent;
	}

	return SkipReason;
}


void FInstallBundleSourcePlatformChunkInstall::RequestUpdateContent(FRequestUpdateContentBundleContext Context)
{
	LOG_SOURCE_CHUNKINSTALL_OVERRIDE(Context.LogVerbosityOverride, Display, TEXT("Requesting Bundle %s"), *Context.BundleName.ToString());

	// sanity check the request
	bool bFailed = false;
	const FBundleInfo* BundleInfo = BundleInfoMap.Find(Context.BundleName);
	if (BundleInfo == nullptr)
	{
		LOG_SOURCE_CHUNKINSTALL_OVERRIDE(Context.LogVerbosityOverride, Display, TEXT("Bundle %s is unknown"), *Context.BundleName.ToString());
		bFailed = true;
	}
	else if (!NamedChunks.Contains(BundleInfo->NamedChunk))
	{
		LOG_SOURCE_CHUNKINSTALL_OVERRIDE(Context.LogVerbosityOverride, Display, TEXT("Bundle %s is not named chunk"), *Context.BundleName.ToString());
		bFailed = true;
	}
	else if (ContentRequests.ContainsByPredicate([BundleName = Context.BundleName](const FContentRequestRef& ContentRequest) { return ContentRequest->BundleName == BundleName; }))
	{
		LOG_SOURCE_CHUNKINSTALL_OVERRIDE(Context.LogVerbosityOverride, Error, TEXT("Bundle %s install already in progress (named chunk: %s)"), *Context.BundleName.ToString(), *BundleInfo->NamedChunk.ToString());
		bFailed = true;
	}
	else if (PlatformChunkInstall->GetNamedChunkLocation(BundleInfo->NamedChunk) == EChunkLocation::LocalFast)
	{
		LOG_SOURCE_CHUNKINSTALL_OVERRIDE(Context.LogVerbosityOverride, Display, TEXT("Bundle request %s finished. (named chunk %s already installed)"), *Context.BundleName.ToString(), *BundleInfo->NamedChunk.ToString() );

		// send the completion callback immediately
		FInstallBundleSourceUpdateContentResultInfo ResultInfo;
		ResultInfo.BundleName = Context.BundleName;
		ResultInfo.Result = EInstallBundleResult::OK;
		ResultInfo.ContentPaths = BundleInfo->FilePaths;
		ResultInfo.bContentWasInstalled = false;

		InstallBundleUtil::FConfigMountOptions ConfigMountOptions;
		InstallBundleUtil::GetMountOptionsFromConfig(FNameBuilder(Context.BundleName), ConfigMountOptions);

		if (ConfigMountOptions.bWithSoftReferences)
		{
			ResultInfo.MountOptions.MountFlags |= FPakMountOptions::EMountFlags::WithSoftReferences;
		}

		Context.CompleteCallback.ExecuteIfBound(AsShared(), MoveTemp(ResultInfo));
	}
	else
	{
		bool bIsInstalling = PlatformChunkInstall->InstallNamedChunk(BundleInfo->NamedChunk);
		if (bIsInstalling)
		{
			FContentRequestRef& NewRequest = ContentRequests.Emplace_GetRef(MakeShared<FContentRequest, ESPMode::ThreadSafe>());
			NewRequest->BundleName = Context.BundleName;
			NewRequest->ContentPaths = BundleInfo->FilePaths;
			NewRequest->LogVerbosityOverride = Context.LogVerbosityOverride;
			NewRequest->CompleteCallback = MoveTemp(Context.CompleteCallback);

			bChunkOrderDirty = true;
		}
		else
		{
			LOG_SOURCE_CHUNKINSTALL_OVERRIDE(Context.LogVerbosityOverride, Display, TEXT("Bundle %s failed to start"), *Context.BundleName.ToString());
			bFailed = true;
		}
	}
	
	if (bFailed)
	{
		// send the failure callback immediately
		FInstallBundleSourceUpdateContentResultInfo ResultInfo;
		ResultInfo.BundleName = Context.BundleName;
		ResultInfo.Result = EInstallBundleResult::OK;
		Context.CompleteCallback.ExecuteIfBound(AsShared(), MoveTemp(ResultInfo));
	}
}


void FInstallBundleSourcePlatformChunkInstall::RequestReleaseContent(FRequestReleaseContentBundleContext Context)
{
	LOG_SOURCE_CHUNKINSTALL_OVERRIDE(Context.LogVerbosityOverride, Display, TEXT("Requesting Release Bundle %s"), *Context.BundleName.ToString());

	// sanity check the request
	bool bFailed = false;
	const FBundleInfo* BundleInfo = BundleInfoMap.Find(Context.BundleName);
	if (BundleInfo == nullptr)
	{
		LOG_SOURCE_CHUNKINSTALL_OVERRIDE(Context.LogVerbosityOverride, Display, TEXT("Bundle %s is unknown"), *Context.BundleName.ToString());
		bFailed = true;
	}
	else if (!NamedChunks.Contains(BundleInfo->NamedChunk))
	{
		LOG_SOURCE_CHUNKINSTALL_OVERRIDE(Context.LogVerbosityOverride, Display, TEXT("Bundle %s is not named chunk"), *Context.BundleName.ToString());
		bFailed = true;
	}
	else if ( ContentReleaseRequests.ContainsByPredicate( [BundleName = Context.BundleName](const FContentReleaseRequestRef& ContentReleaseRequest) { return ContentReleaseRequest->BundleName == BundleName; } ))
	{
		LOG_SOURCE_CHUNKINSTALL_OVERRIDE(Context.LogVerbosityOverride, Error, TEXT("Bundle %s removal already in progress (named chunk: %s)"), *Context.BundleName.ToString(), *BundleInfo->NamedChunk.ToString());
		bFailed = true;
	}
	else if (PlatformChunkInstall->GetNamedChunkLocation(BundleInfo->NamedChunk) == EChunkLocation::NotAvailable)
	{
		LOG_SOURCE_CHUNKINSTALL_OVERRIDE(Context.LogVerbosityOverride, Error, TEXT("Bundle %s is already uninstalled (named chunk: %s)"), *Context.BundleName.ToString(), *BundleInfo->NamedChunk.ToString());

		FInstallBundleSourceReleaseContentResultInfo ResultInfo;
		ResultInfo.BundleName = Context.BundleName;
		ResultInfo.Result = EInstallBundleReleaseResult::OK;
		ResultInfo.bContentWasRemoved = true;
		Context.CompleteCallback.ExecuteIfBound(AsShared(), MoveTemp(ResultInfo));
	}
	else if (!EnumHasAnyFlags(Context.Flags, EInstallBundleReleaseRequestFlags::RemoveFilesIfPossible))
	{
		// caller does not want the content removed
		bFailed = true;
	}
	else
	{
		bool bIsUninstalling = PlatformChunkInstall->UninstallNamedChunk(BundleInfo->NamedChunk);
		if (bIsUninstalling)
		{
			FContentReleaseRequestRef& NewRequest = ContentReleaseRequests.Emplace_GetRef(MakeShared<FContentReleaseRequest, ESPMode::ThreadSafe>());
			NewRequest->BundleName = Context.BundleName;
			NewRequest->LogVerbosityOverride = Context.LogVerbosityOverride;
			NewRequest->CompleteCallback = MoveTemp(Context.CompleteCallback);
		}
		else
		{
			LOG_SOURCE_CHUNKINSTALL_OVERRIDE(Context.LogVerbosityOverride, Display, TEXT("Bundle remove %s failed"), *Context.BundleName.ToString());
			bFailed = true;
		}
	}

	if (bFailed)
	{
		// send the failure callback immediately
		FInstallBundleSourceReleaseContentResultInfo ResultInfo;
		ResultInfo.BundleName = Context.BundleName;
		ResultInfo.Result = EInstallBundleReleaseResult::OK;
		Context.CompleteCallback.ExecuteIfBound(AsShared(), MoveTemp(ResultInfo));
	}
}


void FInstallBundleSourcePlatformChunkInstall::CancelBundles(TArrayView<const FName> BundleNames)
{
	CSV_SCOPED_TIMING_STAT(InstallBundleManager, FInstallBundleSourcePlatformChunkInstall_CancelBundles);

	for ( FName BundleName : BundleNames)
	{
		FContentRequestRef* ContentRequestPtr = ContentRequests.FindByPredicate( [BundleName](const FContentRequestRef& ContentRequest) {  return ContentRequest->BundleName == BundleName; } );
		if (ContentRequestPtr != nullptr)
		{
			FContentRequestRef ContentRequest = (*ContentRequestPtr);
			ContentRequest->bCancelled = true;

			const FBundleInfo* BundleInfo = BundleInfoMap.Find(BundleName);
			if (BundleInfo != nullptr)
			{
				bool bCancelled = PlatformChunkInstall->CancelNamedChunkInstall(BundleInfo->NamedChunk);
				LOG_SOURCE_CHUNKINSTALL(Display, TEXT("Named chunk %s cancellation for bundle %s returned %s"), *BundleInfo->NamedChunk.ToString(), *BundleName.ToString(), *LexToString(bCancelled) );
			}
		}
	}
}


TOptional<FInstallBundleSourceProgress> FInstallBundleSourcePlatformChunkInstall::GetBundleProgress(FName BundleName) const
{
	TOptional<FInstallBundleSourceProgress> Status;

	const FBundleInfo* BundleInfo = BundleInfoMap.Find(BundleName);
	if (BundleInfo != nullptr)
	{
		float ProgressPercent = PlatformChunkInstall->GetNamedChunkProgress(BundleInfo->NamedChunk, EChunkProgressReportingType::PercentageComplete);

		Status.Emplace();
		Status->BundleName = BundleName;
		Status->Install_Percent = (ProgressPercent / 100.0f);
	}

	return Status;
}

void FInstallBundleSourcePlatformChunkInstall::OnNamedChunkInstall( const FNamedChunkCompleteCallbackParam& Param )
{
	// see if this named chunk was being installed by any active requests
	for ( FContentRequestRef Request : ContentRequests)
	{
		if (GetNamedChunkForBundle(Request->BundleName) != Param.NamedChunk)
		{
			continue;
		}

		if (Request->bCancelled)
		{
			LOG_SOURCE_CHUNKINSTALL_OVERRIDE(Request->LogVerbosityOverride, Display, TEXT("Bundle %s cancelled"), *Request->BundleName.ToString());

			FInstallBundleSourceUpdateContentResultInfo ResultInfo;
			ResultInfo.BundleName = Request->BundleName;
			ResultInfo.Result = EInstallBundleResult::UserCancelledError;

			Request->CompleteCallback.ExecuteIfBound(AsShared(), MoveTemp(ResultInfo));
		}
		else
		{
			LOG_SOURCE_CHUNKINSTALL_OVERRIDE(Request->LogVerbosityOverride, Display, TEXT("Bundle request %s finished. IsInstalled: %s, Succeeded: %s"), *Request->BundleName.ToString(), *LexToString(Param.bIsInstalled), *LexToString(Param.bHasSucceeded) );

			// send the completion callback
			FInstallBundleSourceUpdateContentResultInfo ResultInfo;
			ResultInfo.BundleName = Request->BundleName;
			ResultInfo.Result = EInstallBundleResult::OK;
			if (Param.bIsInstalled && Param.bHasSucceeded && Request->ContentPaths.Num() > 0)
			{
				ResultInfo.ContentPaths = Request->ContentPaths;
				ResultInfo.bContentWasInstalled = true;
			}

			InstallBundleUtil::FConfigMountOptions ConfigMountOptions;
			InstallBundleUtil::GetMountOptionsFromConfig(FNameBuilder(Request->BundleName), ConfigMountOptions);

			if (ConfigMountOptions.bWithSoftReferences)
			{
				ResultInfo.MountOptions.MountFlags |= FPakMountOptions::EMountFlags::WithSoftReferences;
			}

			Request->CompleteCallback.ExecuteIfBound(AsShared(), MoveTemp(ResultInfo));
		}

		// mark the request for removal next tick
		Request->bInProgress = false;
	}

	// see if this named chunk was being released by any active requests
	for (FContentReleaseRequestRef Request : ContentReleaseRequests)
	{
		if (GetNamedChunkForBundle(Request->BundleName) != Param.NamedChunk)
		{
			continue;
		}

		LOG_SOURCE_CHUNKINSTALL_OVERRIDE(Request->LogVerbosityOverride, Display, TEXT("Bundle remove request %s finished. IsInstalled: %s, Succeeded: %s"), *Request->BundleName.ToString(), *LexToString(Param.bIsInstalled), *LexToString(Param.bHasSucceeded) );

		// send the completion callback
		FInstallBundleSourceReleaseContentResultInfo ResultInfo;
		ResultInfo.BundleName = Request->BundleName;
		ResultInfo.Result = EInstallBundleReleaseResult::OK;
		ResultInfo.bContentWasRemoved = !Param.bIsInstalled && Param.bHasSucceeded;
		Request->CompleteCallback.ExecuteIfBound(AsShared(), MoveTemp(ResultInfo));

		// mark the request for removal next tick
		Request->bInProgress = false;
	}
}


FName FInstallBundleSourcePlatformChunkInstall::GetNamedChunkForBundle(FName BundleName) const
{
	const FBundleInfo* BundleInfo = BundleInfoMap.Find(BundleName);
	if (BundleInfo != nullptr)
	{
		return BundleInfo->NamedChunk;
	}

	return NAME_None;
}

#endif //WITH_PLATFORM_INSTALL_BUNDLE_SOURCE
