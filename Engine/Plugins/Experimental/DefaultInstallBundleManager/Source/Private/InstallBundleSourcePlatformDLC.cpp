// Copyright Epic Games, Inc. All Rights Reserved.

#include "InstallBundleSourcePlatformDLC.h"
#if WITH_PLATFORM_DLC_INSTALL_BUNDLE_SOURCE
#include "DefaultInstallBundleManagerPrivate.h"

#include "InstallBundleManagerUtil.h"
#include "InstallBundleManagerInterface.h"

#include "Misc/CommandLine.h"
#include "Misc/ConfigCacheIni.h"
#include "Algo/AnyOf.h"
#include "Algo/Find.h"
#include "Containers/Ticker.h"
#include "IPlatformFilePak.h"
#include "Stats/Stats.h"

#define LOG_SOURCE_DLC(Verbosity, Format, ...) LOG_INSTALL_BUNDLE_MAN(Verbosity, TEXT("InstallBundleSourcePlatformDLC: ") Format, ##__VA_ARGS__)

#define LOG_SOURCE_DLC_OVERRIDE(VerbosityOverride, Verbosity, Format, ...) LOG_INSTALL_BUNDLE_MAN_OVERRIDE(VerbosityOverride, Verbosity, TEXT("InstallBundleSourcePlatformDLC: ") Format, ##__VA_ARGS__)

FInstallBundleSourcePlatformDLC::FInstallBundleSourcePlatformDLC(TSharedPtr<IPlatformDLC> InPlatformDLC)
	: PlatformDLC(InPlatformDLC)
{
	TickHandle = FTSTicker::GetCoreTicker().AddTicker(FTickerDelegate::CreateRaw(this, &FInstallBundleSourcePlatformDLC::Tick));

	DLCNotificationDelegateHandle = PlatformDLC->OnNotification().AddRaw(this, &FInstallBundleSourcePlatformDLC::OnDLCNotification);
}


FInstallBundleSourcePlatformDLC::~FInstallBundleSourcePlatformDLC()
{
	PlatformDLC->OnNotification().Remove(DLCNotificationDelegateHandle);
	DLCNotificationDelegateHandle.Reset();

	FTSTicker::RemoveTicker(TickHandle);
	TickHandle.Reset();

	// Cleanup any Async tasks
	InstallBundleUtil::CleanupInstallBundleAsyncIOTasks(GeneralAsyncTasks);
}

FInstallBundleSourceType FInstallBundleSourcePlatformDLC::GetSourceType() const
{
	return FInstallBundleSourceType(TEXT("PlatformDLC"));
}

bool FInstallBundleSourcePlatformDLC::Tick(float dt)
{
	QUICK_SCOPE_CYCLE_COUNTER(STAT_FInstallBundleSourcePlatformDLC_Tick);

	CSV_SCOPED_TIMING_STAT(InstallBundleManager, FInstallBundleSourcePlatformDLC_Tick);

	InstallBundleUtil::FinishInstallBundleAsyncIOTasks(GeneralAsyncTasks);

	// remove finished requests
	ContentReleaseRequests.RemoveAll([](const FDLCContentReleaseRequestRef& Request) { return !Request->bInProgress; });
	ContentRequests.RemoveAll([](const FDLCContentRequestRef& Request) { return !Request->bInProgress; });

	return true;
}

FInstallBundleSourceInitInfo FInstallBundleSourcePlatformDLC::Init(TSharedRef<InstallBundleUtil::FContentRequestStatsMap> InRequestStats, TSharedPtr<IAnalyticsProviderET> InAnalyticsProvider, TSharedPtr<InstallBundleUtil::PersistentStats::FPersistentStatContainerBase> InPersistentStatsContainer)
{
	CSV_SCOPED_TIMING_STAT(InstallBundleManager, FInstallBundleSourcePlatformDLC_Init);
	FInstallBundleSourceInitInfo InitInfo;

	// Allow the PlatformDLC bundle source to opt-out and use a fallback bundle source. This is to support platforms
	// that may only support DLC in certain install scenarios (e.g., running from an installed package).
	if (!PlatformDLC->IsAvailable())
	{
		LOG_SOURCE_DLC(Display, TEXT("PlatformDLC is unavailable. Attempting to fallback to next bundle source."));

		InitInfo.Result = EInstallBundleManagerInitResult::ConfigurationError;
		InitInfo.bShouldUseFallbackSource = true;

		return InitInfo;
	}
	else
	{
		InitInfo = FInstallBundleSourcePlatformBase::Init(InRequestStats, InAnalyticsProvider, InPersistentStatsContainer);
	}

	return InitInfo;
}


void FInstallBundleSourcePlatformDLC::AsyncInit(FInstallBundleSourceInitDelegate Callback)
{
	CSV_SCOPED_TIMING_STAT(InstallBundleManager, FInstallBundleSourcePlatformDLC_AsyncInit);

	// Handle retrying init if recoverable
	if (InitState == EInstallBundleManagerInitState::Failed)
	{
		LOG_SOURCE_DLC(Warning, TEXT("Retrying initialization"));
		InitState = EInstallBundleManagerInitState::NotInitialized;
	}

	struct AsyncInitContext
	{
		EInstallBundleManagerInitResult InitResult = EInstallBundleManagerInitResult::OK;

		TSet<FName> DLCNames;
		TMap<FName, FBundleInfo> BundleInfoMap;
		TMap<FName, TArray<FString>> BundleToSubsetMap;
		TArray<FString> PakSearchDirs;
		TSharedPtr<IPlatformDLC> PlatformDLC;
	};


	// create and initialize the async init context
	TSharedPtr<AsyncInitContext, ESPMode::ThreadSafe> Context = MakeShared<AsyncInitContext, ESPMode::ThreadSafe>();
	Context->PlatformDLC = PlatformDLC;
	

	auto AsyncInit = [Context]() mutable
	{
		// make sure there's a config
		const FConfigFile* InstallBundleConfig = GConfig->FindConfigFile(GInstallBundleIni);
		if (!InstallBundleConfig)
		{
			Context->InitResult = EInstallBundleManagerInitResult::ConfigurationError;
			return;
		}

		// cache all DLCs
		Context->DLCNames.Append( Context->PlatformDLC->GetAllDLCNames() );
		LOG_SOURCE_DLC(Display, TEXT("Platform DLC has %d DLCs"), Context->DLCNames.Num() );

		// create bundles & chunk data
		TSet<FName> KnownDLCNames;
		TMap<FName, TArray<FString>> DLCNameFilePaths;
		FString RootDir = FPaths::RootDir();
		bool bHasAbsoluteRootDir = !FPaths::IsRelative(RootDir);

		for (const TPair<FString, FConfigSection>& Pair : *InstallBundleConfig)
		{
			// make sure this is a suitable platform bundle
			const FString& Section = Pair.Key;
			if (!Section.StartsWith(InstallBundleUtil::GetInstallBundleSectionPrefix()))
			{
				continue;
			}
			
			// see if this bundle has platform DLC information
			FString PlatformDLCName;
			InstallBundleConfig->GetString(*Section, TEXT("PlatformDLCName"), PlatformDLCName);
			if (PlatformDLCName.IsEmpty())
			{
				continue;
			}

			// minor hack: DLC files are not known until they are mounted but QueryPersistentBundleInfo needs this pre-mount... have to bake it in for now
			bool bContainsIoStoreOnDemandToc = false;
			InstallBundleConfig->GetBool(*Section, TEXT("ContainsIoStoreOnDemandToc"), bContainsIoStoreOnDemandToc);

			// create bundle info
			FName BundleName( *Section.RightChop(InstallBundleUtil::GetInstallBundleSectionPrefix().Len()));
			FBundleInfo& BundleInfo = Context->BundleInfoMap.Add(BundleName);
			BundleInfo.DLCName = FName(PlatformDLCName);
			BundleInfo.Priority = EInstallBundlePriority::Normal;
			BundleInfo.bContainsIoStoreOnDemandToc = bContainsIoStoreOnDemandToc;

			FString PriorityString;
			InstallBundleConfig->GetString(*Section, TEXT("Priority"), PriorityString);
			LexTryParseString(BundleInfo.Priority, *PriorityString);

			TArray<FString> BundleSubsets;
			InstallBundleConfig->GetArray(*Section, TEXT("Subsets"), BundleSubsets);
			if (!BundleSubsets.IsEmpty())
			{
				Context->BundleToSubsetMap.Add(BundleName, MoveTemp(BundleSubsets));
			}

			if (KnownDLCNames.Contains(BundleInfo.DLCName))
			{
				continue;
			}
			KnownDLCNames.Add(BundleInfo.DLCName);

			LOG_SOURCE_DLC(Display, TEXT("Initialized DLC %s for bundle %s %s"), *PlatformDLCName, *BundleName.ToString(), (Context->DLCNames.Contains(BundleInfo.DLCName)) ? TEXT("") : TEXT("...not DLC!") );
		}

		// collect pak search directories
		TArray<FString> PakFolders;
		FPakPlatformFile::GetPakFolders(FCommandLine::Get(), PakFolders);
		
		Context->PakSearchDirs.Empty(PakFolders.Num());
		for (FString& PakFolder : PakFolders)
		{
			if (bHasAbsoluteRootDir && FPaths::IsRelative(PakFolder))
			{
				Context->PakSearchDirs.Add(MoveTemp(PakFolder));
			}
			else if (PakFolder.StartsWith(RootDir))
			{
				verify(FPaths::MakePathRelativeTo(Context->PakSearchDirs.Add_GetRef(MoveTemp(PakFolder)), *RootDir));
			}
		}

	};

	auto OnAsyncInitComplete = [this, Context, OnInitCompleteCallback = MoveTemp(Callback)]() mutable
	{
		check(IsInGameThread());

		FInstallBundleSourceAsyncInitInfo InitInfo;
		InitInfo.Result = Context->InitResult;

		if (InitInfo.Result == EInstallBundleManagerInitResult::OK)
		{
			// initialization succeeded - store the initialized data
			BundleInfoMap = MoveTemp(Context->BundleInfoMap);
			BundleToSubsetMap = MoveTemp(Context->BundleToSubsetMap);
			PakSearchDirs = MoveTemp(Context->PakSearchDirs);
			DLCNames = MoveTemp(Context->DLCNames);

			// remove regex entries for bundles not associated with a known platform DLC
			auto IsNotKnownDLCBundle = [this](const TPair<FString, TArray<FRegexPattern>>& Entry)
			{
				return !BundleInfoMap.Contains(FName(*Entry.Key));
			};
			BundleRegexList.RemoveAll(IsNotKnownDLCBundle);
			BundleRegexList.Shrink();
			BundleSubsetRegexList.RemoveAll(IsNotKnownDLCBundle);
			BundleSubsetRegexList.Shrink();

			for (const auto& BundleInfo : BundleInfoMap)
			{
				LOG_SOURCE_DLC(Display, TEXT("%32s -> %-32s"), *BundleInfo.Key.ToString(), *BundleInfo.Value.DLCName.ToString());
			}
			LOG_SOURCE_DLC(Display, TEXT("Initialization completed - %d bundles, %d DLCs"), BundleInfoMap.Num(), DLCNames.Num() );
			InitState = EInstallBundleManagerInitState::Succeeded;
		}
		else
		{
			LOG_SOURCE_DLC(Error, TEXT("Initialization Failed - %s"), LexToString(InitInfo.Result));
			InitState = EInstallBundleManagerInitState::Failed;
		}

		LOG_SOURCE_DLC(Display, TEXT("Fire Init Analytic: %s"), LexToString(InitInfo.Result));
		InstallBundleManagerAnalytics::FireEvent_InitBundleSourcePlatformDLCComplete(AnalyticsProvider.Get(), LexToString(InitInfo.Result));

		OnInitCompleteCallback.Execute(AsShared(), MoveTemp(InitInfo));
	};

	PlatformDLC->RegisterInitializationCallback([this, AsyncInit = MoveTemp(AsyncInit), OnAsyncInitComplete = MoveTemp(OnAsyncInitComplete)]() mutable
	{
		LOG_SOURCE_DLC(Display, TEXT("Platform DLC init succeeded"));
		InstallBundleUtil::StartInstallBundleAsyncIOTask(GeneralAsyncTasks, MoveTemp(AsyncInit), MoveTemp(OnAsyncInitComplete));
	});
}


bool FInstallBundleSourcePlatformDLC::QueryPersistentBundleInfo(FInstallBundleSourcePersistentBundleInfo& SourceBundleInfo) const
{
	// lookup bundle info
	const FBundleInfo* BundleInfo = BundleInfoMap.Find(SourceBundleInfo.BundleName);
	if (BundleInfo == nullptr)
	{
		return false;
	}

	SourceBundleInfo.bContainsIoStoreOnDemandToc = BundleInfo->bContainsIoStoreOnDemandToc;

	// assume the startup bundle is installed. It cannot be uninstalled
	if (SourceBundleInfo.bIsStartup)
	{
		SourceBundleInfo.BundleContentState = EInstallBundleInstallState::UpToDate;
		return true;
	}

	// lookup the chunk data
	uint64 CurrentDownloadSize = 0;
	uint64 TotalDownloadSize = 0;
	if (PlatformDLC->GetDownloadSize(BundleInfo->DLCName, CurrentDownloadSize, TotalDownloadSize))
	{
		SourceBundleInfo.CurrentInstallSize = CurrentDownloadSize;
		SourceBundleInfo.FullInstallSize = TotalDownloadSize;
	}

	const bool bIsMounted = PlatformDLC->GetState(BundleInfo->DLCName) == IPlatformDLC::EState::Mounted;
	SourceBundleInfo.BundleContentState = bIsMounted ? EInstallBundleInstallState::UpToDate : EInstallBundleInstallState::NotInstalled;

	return true;
}


void FInstallBundleSourcePlatformDLC::GetContentState(TArrayView<const FName> BundleNames, EInstallBundleGetContentStateFlags Flags, FInstallBundleGetContentStateDelegate Callback)
{
	CSV_SCOPED_TIMING_STAT(InstallBundleManager, FInstallBundleSourcePlatformDLC_GetContentState);

	FInstallBundleCombinedContentState State;
	State.CurrentVersion.Add(GetSourceType(), InstallBundleUtil::GetAppVersion());

	// capture the install state and version for all bundles, and record the remaining download size
	TMap<FName,uint64> BundleToRemaingDownloadSize;
	for (const FName& BundleName : BundleNames)
	{
		const FBundleInfo* BundleInfo = BundleInfoMap.Find(BundleName);
		if (BundleInfo != nullptr)
		{
			FInstallBundleContentState& IndividualBundleState = State.IndividualBundleStates.Add(BundleName);
			const bool bIsMounted = PlatformDLC->GetState(BundleInfo->DLCName) == IPlatformDLC::EState::Mounted;
			if (bIsMounted)
			{
				IndividualBundleState.State = EInstallBundleInstallState::UpToDate;
				IndividualBundleState.Version.Add(GetSourceType(), InstallBundleUtil::GetAppVersion());
			}
			else
			{
				// DLC is either not installed, or it's not a valid DLC (also meaning it isn't installed)... 
				// an invalid DLC likely means there is an entry for it in the bundle ini, but the platform chunk it refers to does not exist
				IndividualBundleState.State = EInstallBundleInstallState::NotInstalled;
				IndividualBundleState.Version.Add(GetSourceType(), TEXT(""));
			}

			uint64 RemainingDownloadSize = 0;
			uint64 CurrentDownloadSize = 0;
			uint64 TotalDownloadSize = 0;
			if (PlatformDLC->GetDownloadSize(BundleInfo->DLCName, CurrentDownloadSize, TotalDownloadSize))
			{
				check( CurrentDownloadSize <= TotalDownloadSize);
				RemainingDownloadSize = (TotalDownloadSize - CurrentDownloadSize);

				State.ContentSize.DownloadSize += RemainingDownloadSize;
				State.ContentSize.CurrentSizeOnDiskOtherDirs += CurrentDownloadSize;
				State.ContentSize.SpaceRequiredForInstallOtherDirs += RemainingDownloadSize;
			}
			BundleToRemaingDownloadSize.Add(BundleName, RemainingDownloadSize);
		}
	}

	// compute the download weight for all of the known bundles - higher weight is a bigger download
	for (TTuple<FName,FInstallBundleContentState>& BundleStatePair : State.IndividualBundleStates)
	{
		if (State.ContentSize.DownloadSize == 0)
		{
			BundleStatePair.Value.Weight = 1.0f / BundleNames.Num();
			continue;
		}

		double Weight = (double)BundleToRemaingDownloadSize[BundleStatePair.Key] / (double)State.ContentSize.DownloadSize;
		BundleStatePair.Value.Weight = FMath::Max(Weight, InstallBundleUtil::MinimumBundleWeight); // this Max() does mean the total weight will be > 1.0 but all other bundle sources do it this way too
	}

	Callback.ExecuteIfBound(MoveTemp(State));
}


EInstallBundleSourceBundleSkipReason FInstallBundleSourcePlatformDLC::GetBundleSkipReason(FName BundleName) const
{
	return EInstallBundleSourceBundleSkipReason::None;
}


void FInstallBundleSourcePlatformDLC::RequestUpdateContent(FRequestUpdateContentBundleContext Context)
{
	LOG_SOURCE_DLC_OVERRIDE(Context.LogVerbosityOverride, Display, TEXT("Requesting Bundle %s"), *Context.BundleName.ToString());

	// sanity check the request
	bool bFailed = false;
	const FBundleInfo* BundleInfo = BundleInfoMap.Find(Context.BundleName);
	if (BundleInfo == nullptr)
	{
		LOG_SOURCE_DLC_OVERRIDE(Context.LogVerbosityOverride, Display, TEXT("Bundle %s is unknown"), *Context.BundleName.ToString());
		bFailed = true;
	}
	else if (!DLCNames.Contains(BundleInfo->DLCName))
	{
		LOG_SOURCE_DLC_OVERRIDE(Context.LogVerbosityOverride, Display, TEXT("Bundle %s is not a known DLC"), *Context.BundleName.ToString());
		bFailed = true;
	}
	else if (ContentRequests.ContainsByPredicate([BundleName = Context.BundleName](const FDLCContentRequestRef& ContentRequest) { return ContentRequest->BundleName == BundleName; }))
	{
		LOG_SOURCE_DLC_OVERRIDE(Context.LogVerbosityOverride, Error, TEXT("Bundle %s install already in progress (DLC: %s)"), *Context.BundleName.ToString(), *BundleInfo->DLCName.ToString());
		bFailed = true;
	}
	else if (IPlatformDLC::EState State = PlatformDLC->GetState(BundleInfo->DLCName); State == IPlatformDLC::EState::Mounted)
	{
		LOG_SOURCE_DLC_OVERRIDE(Context.LogVerbosityOverride, Display, TEXT("Bundle request %s finished. (DLC %s already installed)"), *Context.BundleName.ToString(), *BundleInfo->DLCName.ToString() );

		// send the completion callback immediately
		FInstallBundleSourceUpdateContentResultInfo ResultInfo;
		ResultInfo.BundleName = Context.BundleName;
		ResultInfo.Result = EInstallBundleResult::OK;
		ResultInfo.ContentPaths = GetContentPathsForDLC(BundleInfo->DLCName, Context.BundleName);
		ResultInfo.bContentWasInstalled = (ResultInfo.ContentPaths.Num() > 0);

		InstallBundleUtil::FConfigMountOptions ConfigMountOptions;
		InstallBundleUtil::GetMountOptionsFromConfig(FNameBuilder(Context.BundleName), ConfigMountOptions);

		if (ConfigMountOptions.bWithSoftReferences)
		{
			ResultInfo.MountOptions.MountFlags |= FPakMountOptions::EMountFlags::WithSoftReferences;
		}

		Context.CompleteCallback.ExecuteIfBound(AsShared(), MoveTemp(ResultInfo));
	}
	else if (State == IPlatformDLC::EState::Downloading || State == IPlatformDLC::EState::Mounting)
	{
		FDLCContentRequestRef& NewRequest = ContentRequests.Emplace_GetRef(MakeShared<FDLCContentRequest, ESPMode::ThreadSafe>());
		NewRequest->BundleName = Context.BundleName;
		NewRequest->LogVerbosityOverride = Context.LogVerbosityOverride;
		NewRequest->CompleteCallback = MoveTemp(Context.CompleteCallback);
	}
	else
	{
		FPlatformUserId StoreUserId = GetDLCStoreUserId();
		if (StoreUserId != PLATFORMUSERID_NONE)
		{
			PlatformDLC->SetStoreUser(StoreUserId);
		}
		bool bIsInstalling = PlatformDLC->Download(BundleInfo->DLCName);
		if (bIsInstalling)
		{
			FDLCContentRequestRef& NewRequest = ContentRequests.Emplace_GetRef(MakeShared<FDLCContentRequest, ESPMode::ThreadSafe>());
			NewRequest->BundleName = Context.BundleName;
			NewRequest->LogVerbosityOverride = Context.LogVerbosityOverride;
			NewRequest->CompleteCallback = MoveTemp(Context.CompleteCallback);
		}
		else
		{
			LOG_SOURCE_DLC_OVERRIDE(Context.LogVerbosityOverride, Display, TEXT("Bundle %s failed to start"), *Context.BundleName.ToString());
			bFailed = true;
		}
	}
	
	if (bFailed)
	{
		// send the failure callback immediately
		FInstallBundleSourceUpdateContentResultInfo ResultInfo;
		ResultInfo.BundleName = Context.BundleName;
		ResultInfo.Result = EInstallBundleResult::OK; // the request itself succeeded, bContentWasInstalled is false here
		Context.CompleteCallback.ExecuteIfBound(AsShared(), MoveTemp(ResultInfo));
	}
}


void FInstallBundleSourcePlatformDLC::RequestReleaseContent(FRequestReleaseContentBundleContext Context)
{
	LOG_SOURCE_DLC_OVERRIDE(Context.LogVerbosityOverride, Display, TEXT("Requesting Release Bundle %s"), *Context.BundleName.ToString());

	bool bUninstallRequested = EnumHasAnyFlags(Context.Flags, EInstallBundleReleaseRequestFlags::RemoveFilesIfPossible);

	// sanity check the request
	bool bFailed = false;
	const FBundleInfo* BundleInfo = BundleInfoMap.Find(Context.BundleName);
	if (BundleInfo == nullptr)
	{
		LOG_SOURCE_DLC_OVERRIDE(Context.LogVerbosityOverride, Display, TEXT("Bundle %s is unknown"), *Context.BundleName.ToString());
		bFailed = true;
	}
	else if (!DLCNames.Contains(BundleInfo->DLCName))
	{
		LOG_SOURCE_DLC_OVERRIDE(Context.LogVerbosityOverride, Display, TEXT("Bundle %s is not DLC"), *Context.BundleName.ToString());
		bFailed = true;
	}
	else if ( ContentReleaseRequests.ContainsByPredicate( [BundleName = Context.BundleName](const FDLCContentReleaseRequestRef& ContentReleaseRequest) { return ContentReleaseRequest->BundleName == BundleName; } ))
	{
		LOG_SOURCE_DLC_OVERRIDE(Context.LogVerbosityOverride, Error, TEXT("Bundle %s removal already in progress (DLC: %s)"), *Context.BundleName.ToString(), *BundleInfo->DLCName.ToString());
		bFailed = true;
	}
	else if ( !bUninstallRequested && PlatformDLC->GetState(BundleInfo->DLCName) != IPlatformDLC::EState::Mounted)
	{
		LOG_SOURCE_DLC_OVERRIDE(Context.LogVerbosityOverride, Error, TEXT("Bundle %s is already released (DLC: %s is not mounted)"), *Context.BundleName.ToString(), *BundleInfo->DLCName.ToString());
		bFailed = true;
	}
	else if (bUninstallRequested && PlatformDLC->GetState(BundleInfo->DLCName) == IPlatformDLC::EState::NotInstalled)
	{
		LOG_SOURCE_DLC_OVERRIDE(Context.LogVerbosityOverride, Error, TEXT("Bundle %s is already released (DLC: %s is not installed)"), *Context.BundleName.ToString(), *BundleInfo->DLCName.ToString());

		FInstallBundleSourceReleaseContentResultInfo ResultInfo;
		ResultInfo.BundleName = Context.BundleName;
		ResultInfo.Result = EInstallBundleReleaseResult::OK;
		ResultInfo.bContentWasRemoved = true;
		Context.CompleteCallback.ExecuteIfBound(AsShared(), MoveTemp(ResultInfo));
	}
	else if (IPlatformDLC::EState State = PlatformDLC->GetState(BundleInfo->DLCName); State == IPlatformDLC::EState::Mounted)
	{
		bool bIsUnmounting = PlatformDLC->Unmount(BundleInfo->DLCName);
		if (bIsUnmounting)
		{
			FDLCContentReleaseRequestRef& NewRequest = ContentReleaseRequests.Emplace_GetRef(MakeShared<FDLCContentReleaseRequest, ESPMode::ThreadSafe>());
			NewRequest->BundleName = Context.BundleName;
			NewRequest->LogVerbosityOverride = Context.LogVerbosityOverride;
			NewRequest->CompleteCallback = MoveTemp(Context.CompleteCallback);
			NewRequest->bUninstallRequested = bUninstallRequested; // whether to follow this up with an uninstall
		}
		else
		{
			bFailed = true;
			LOG_SOURCE_DLC_OVERRIDE(Context.LogVerbosityOverride, Display, TEXT("Bundle remove %s failed (cannot unmount DLC %s)"), *Context.BundleName.ToString(), *BundleInfo->DLCName.ToString());
		}
	}
	else if (State == IPlatformDLC::EState::Unmounting)
	{
		FDLCContentReleaseRequestRef& NewRequest = ContentReleaseRequests.Emplace_GetRef(MakeShared<FDLCContentReleaseRequest, ESPMode::ThreadSafe>());
		NewRequest->BundleName = Context.BundleName;
		NewRequest->LogVerbosityOverride = Context.LogVerbosityOverride;
		NewRequest->CompleteCallback = MoveTemp(Context.CompleteCallback);
		NewRequest->bUninstallRequested = bUninstallRequested; // whether to follow this up with an uninstall
	}
	else if (bUninstallRequested)
	{
		bool bIsUninstalling = State == IPlatformDLC::EState::Uninstalling || PlatformDLC->Uninstall(BundleInfo->DLCName);
		if (bIsUninstalling)
		{
			FDLCContentReleaseRequestRef& NewRequest = ContentReleaseRequests.Emplace_GetRef(MakeShared<FDLCContentReleaseRequest, ESPMode::ThreadSafe>());
			NewRequest->BundleName = Context.BundleName;
			NewRequest->LogVerbosityOverride = Context.LogVerbosityOverride;
			NewRequest->CompleteCallback = MoveTemp(Context.CompleteCallback);
		}
		else
		{
			bFailed = true;
			LOG_SOURCE_DLC_OVERRIDE(Context.LogVerbosityOverride, Display, TEXT("Bundle remove %s failed (cannot uninstall DLC %s)"), *Context.BundleName.ToString(), *BundleInfo->DLCName.ToString());
		}
	}

	if (bFailed)
	{
		// send the failure callback immediately
		FInstallBundleSourceReleaseContentResultInfo ResultInfo;
		ResultInfo.BundleName = Context.BundleName;
		ResultInfo.Result = EInstallBundleReleaseResult::OK; // the request itself succeeded. bContentWasRemoved contains the actual state
		Context.CompleteCallback.ExecuteIfBound(AsShared(), MoveTemp(ResultInfo));
	}
}


void FInstallBundleSourcePlatformDLC::CancelBundles(TArrayView<const FName> BundleNames)
{
	CSV_SCOPED_TIMING_STAT(InstallBundleManager, FInstallBundleSourcePlatformDLC_CancelBundles);

	for ( FName BundleName : BundleNames)
	{
		FDLCContentRequestRef* ContentRequestPtr = ContentRequests.FindByPredicate( [BundleName](const FDLCContentRequestRef& ContentRequest) {  return ContentRequest->BundleName == BundleName; } );
		if (ContentRequestPtr != nullptr)
		{
			FDLCContentRequestRef ContentRequest = (*ContentRequestPtr);
			ContentRequest->bCancelled = true;
		}
	}
}


TOptional<FInstallBundleSourceProgress> FInstallBundleSourcePlatformDLC::GetBundleProgress(FName BundleName) const
{
	TOptional<FInstallBundleSourceProgress> Status;

	const FBundleInfo* BundleInfo = BundleInfoMap.Find(BundleName);
	if (BundleInfo != nullptr)
	{
		uint64 CurrentDownloadSize = 0;
		uint64 TotalDownloadSize = 0;
		if (PlatformDLC->GetDownloadSize(BundleInfo->DLCName, CurrentDownloadSize, TotalDownloadSize) && TotalDownloadSize > 0)
		{
			Status.Emplace();
			Status->BundleName = BundleName;
			Status->Install_Percent = (float)FMath::GetRangePct(0.0, (double)TotalDownloadSize, (double)CurrentDownloadSize);
		}
	}

	return Status;
}

void FInstallBundleSourcePlatformDLC::OnDLCNotification(FName DLCName, IPlatformDLC::ENotification Notification, bool bSuccess)
{
	// see if this DLC was being installed by any active requests
	if (Notification == IPlatformDLC::ENotification::Downloaded || Notification == IPlatformDLC::ENotification::Mounted)
	{
		for ( FDLCContentRequestRef Request : ContentRequests)
		{
			if (!Request->bInProgress)
			{
				continue;
			}

			if (GetDLCNameForBundle(Request->BundleName) != DLCName)
			{
				continue;
			}

			if (Request->bCancelled)
			{
				LOG_SOURCE_DLC_OVERRIDE(Request->LogVerbosityOverride, Display, TEXT("Bundle %s cancelled"), *Request->BundleName.ToString());

				FInstallBundleSourceUpdateContentResultInfo ResultInfo;
				ResultInfo.BundleName = Request->BundleName;
				ResultInfo.Result = EInstallBundleResult::UserCancelledError;

				Request->CompleteCallback.ExecuteIfBound(AsShared(), MoveTemp(ResultInfo));
				Request->bInProgress = false;
			}
			else
			{
				if (bSuccess && Notification == IPlatformDLC::ENotification::Downloaded)
				{
					bSuccess &= PlatformDLC->Mount(DLCName);
				}
				else if (bSuccess && Notification == IPlatformDLC::ENotification::Mounted)
				{
					LOG_SOURCE_DLC_OVERRIDE(Request->LogVerbosityOverride, Display, TEXT("Bundle request %s succeeded"), *Request->BundleName.ToString() );

					// send the completion callback
					FInstallBundleSourceUpdateContentResultInfo ResultInfo;
					ResultInfo.BundleName = Request->BundleName;
					ResultInfo.Result = EInstallBundleResult::OK;
					ResultInfo.ContentPaths = GetContentPathsForDLC(DLCName, Request->BundleName);
					ResultInfo.bContentWasInstalled = (ResultInfo.ContentPaths.Num() > 0);

					InstallBundleUtil::FConfigMountOptions ConfigMountOptions;
					InstallBundleUtil::GetMountOptionsFromConfig(FNameBuilder(Request->BundleName), ConfigMountOptions);

					if (ConfigMountOptions.bWithSoftReferences)
					{
						ResultInfo.MountOptions.MountFlags |= FPakMountOptions::EMountFlags::WithSoftReferences;
					}

					Request->CompleteCallback.ExecuteIfBound(AsShared(), MoveTemp(ResultInfo));
					Request->bInProgress = false;
				}

				if (!bSuccess)
				{
					LOG_SOURCE_DLC_OVERRIDE(Request->LogVerbosityOverride, Display, TEXT("Bundle request %s failed"), *Request->BundleName.ToString() );

					// send the failure callback
					FInstallBundleSourceUpdateContentResultInfo ResultInfo;
					ResultInfo.BundleName = Request->BundleName;
					ResultInfo.Result = EInstallBundleResult::InstallError;

					Request->CompleteCallback.ExecuteIfBound(AsShared(), MoveTemp(ResultInfo));
					Request->bInProgress = false;
				}
			}
		}
	}

	// see if this DLC was being released by any active requests
	if (Notification == IPlatformDLC::ENotification::Uninstalled || Notification == IPlatformDLC::ENotification::Unmounted)
	{
		for (FDLCContentReleaseRequestRef Request : ContentReleaseRequests)
		{
			if (!Request->bInProgress)
			{
				continue;
			}

			if (GetDLCNameForBundle(Request->BundleName) != DLCName)
			{
				continue;
			}

			if (bSuccess && Notification == IPlatformDLC::ENotification::Unmounted && Request->bUninstallRequested)
			{
				bSuccess &= PlatformDLC->Uninstall(DLCName);
			}
			else if (bSuccess && Notification == IPlatformDLC::ENotification::Unmounted)
			{
				LOG_SOURCE_DLC_OVERRIDE(Request->LogVerbosityOverride, Display, TEXT("Bundle unmount request %s finished"), *Request->BundleName.ToString() );

				// send the completion callback
				FInstallBundleSourceReleaseContentResultInfo ResultInfo;
				ResultInfo.BundleName = Request->BundleName;
				ResultInfo.Result = EInstallBundleReleaseResult::OK;
				ResultInfo.bContentWasRemoved = false;

				Request->CompleteCallback.ExecuteIfBound(AsShared(), MoveTemp(ResultInfo));
				Request->bInProgress = false;
			}
			else if (bSuccess && Notification == IPlatformDLC::ENotification::Uninstalled)
			{
				LOG_SOURCE_DLC_OVERRIDE(Request->LogVerbosityOverride, Display, TEXT("Bundle uninstall request %s finished"), *Request->BundleName.ToString());

				// send the completion callback
				FInstallBundleSourceReleaseContentResultInfo ResultInfo;
				ResultInfo.BundleName = Request->BundleName;
				ResultInfo.Result = EInstallBundleReleaseResult::OK;
				ResultInfo.bContentWasRemoved = true;

				Request->CompleteCallback.ExecuteIfBound(AsShared(), MoveTemp(ResultInfo));
				Request->bInProgress = false;
			}

			if (!bSuccess)
			{
				LOG_SOURCE_DLC_OVERRIDE(Request->LogVerbosityOverride, Display, TEXT("Bundle remove request %s failed"), *Request->BundleName.ToString());

				// send the failure callback
				FInstallBundleSourceReleaseContentResultInfo ResultInfo;
				ResultInfo.BundleName = Request->BundleName;
				ResultInfo.Result = EInstallBundleReleaseResult::OK; // the request itself succeeded. bContentWasRemoved contains the actual state
				ResultInfo.bContentWasRemoved = (PlatformDLC->GetState(DLCName) != IPlatformDLC::EState::NotInstalled);
				
				Request->CompleteCallback.ExecuteIfBound(AsShared(), MoveTemp(ResultInfo));
				Request->bInProgress = false;
			}
		}
	}
}


TArray<FString> FInstallBundleSourcePlatformDLC::GetContentPathsForDLC( FName DLCName, FName BundleName )
{
	FBundleInfo* BundleInfo = BundleInfoMap.Find(BundleName);
	if (BundleInfo != nullptr)
	{
		// see if we've computed these already
		if (BundleInfo->bHasSetFilePaths)
		{
			return BundleInfo->FilePaths;
		}

		// once the DLC is mounted we can find out the content paths
		if (PlatformDLC->GetState(DLCName) == IPlatformDLC::EState::Mounted)
		{
			FString DLCRoot = PlatformDLC->GetRootDirectory(DLCName);
			if (!DLCRoot.IsEmpty())
			{
				TSharedPtr<IInstallBundleManager> BundleManager = IInstallBundleManager::GetPlatformInstallBundleManager();
				TArray<FString>* BundleSubsets = BundleToSubsetMap.Find(BundleName);
				const int32 iSubset = (!BundleSubsets) ? INDEX_NONE : BundleManager->SelectBundleSubset(BundleName.ToString(), *BundleSubsets);

				// DLC root directory is often an unusual path outside of the game's root
				FString NormalizedDLCRoot = DLCRoot;
				FPaths::NormalizeDirectoryName(NormalizedDLCRoot);

				// MakePathRelativeTo expects a trailing / on the path
				if (!NormalizedDLCRoot.EndsWith(TEXT("/")))
				{
					NormalizedDLCRoot += TEXT('/');
				}

				// get all the relevant DLC files
				TArray<FString> DLCFiles;
				for (const FString& PakSearchDir : PakSearchDirs)
				{
					FString AbsSearchDir = FPaths::Combine(DLCRoot, PakSearchDir);
					IPlatformFile::GetPlatformPhysical().FindFilesRecursively(DLCFiles, *AbsSearchDir, TEXT(".pak"));
					IPlatformFile::GetPlatformPhysical().FindFilesRecursively(DLCFiles, *AbsSearchDir, TEXT(".uondemandtoc"));
				}

				// find the ones that should go into this bundle
				for (FString& DLCFile : DLCFiles)
				{
					FString RelativeDLCPath = DLCFile;
					FPaths::MakePathRelativeTo(RelativeDLCPath, *NormalizedDLCRoot);

					FString MatchedBundleName;
					if (InstallBundleUtil::MatchBundleRegex(BundleRegexList, RelativeDLCPath, MatchedBundleName))
					{
						if (FName(MatchedBundleName) != BundleName)
						{
							continue;
						}

						if (!BundleSubsets || !BundleSubsets->IsValidIndex(iSubset) ||
							InstallBundleUtil::MatchBundleSubsetRegex((*BundleSubsets)[iSubset], BundleSubsetRegexList, DLCFile))
						{
							BundleInfo->FilePaths.AddUnique(MoveTemp(DLCFile));
						}
					}
					else
					{
						checkf(false, TEXT("Failed to map chunk file %s to an install bundle"), *DLCFile);
					}
				}

				// sanity check hard-coded hack now we have the files available
				bool bContainsIoStoreOnDemandToc = Algo::AnyOf(
					BundleInfo->FilePaths, [](FStringView File) { return File.EndsWith(TEXTVIEW(".uondemandtoc")); });
				checkf(bContainsIoStoreOnDemandToc == BundleInfo->bContainsIoStoreOnDemandToc, TEXT("configuration mismatch - please update [InstallBundleDefinition %s]:ContainsIoStoreOnDemandToc=%s for this platform"), *BundleName.ToString(), *LexToString(bContainsIoStoreOnDemandToc));

				// don't look up again
				BundleInfo->bHasSetFilePaths = true;
				return BundleInfo->FilePaths;
			}
		}
	}

	// cannot get the file paths
	return TArray<FString>();
}


FName FInstallBundleSourcePlatformDLC::GetDLCNameForBundle(FName BundleName) const
{
	const FBundleInfo* BundleInfo = BundleInfoMap.Find(BundleName);
	if (BundleInfo != nullptr)
	{
		return BundleInfo->DLCName;
	}

	return NAME_None;
}

#endif //WITH_PLATFORM_DLC_INSTALL_BUNDLE_SOURCE

