// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "InstallBundleManagerInterface.h"
#include "InstallBundleManagerUtil.h"
#include "InstallBundleCache.h"
#include "InstallBundleManagerReporting.h"

#include "Experimental/UnifiedError/UnifiedError.h"

#include "Containers/StaticArray.h"
#include "Templates/ValueOrError.h"

#define UE_API DEFAULTINSTALLBUNDLEMANAGER_API

#ifndef INSTALL_BUNDLE_ENABLE_ANALYTICS
	#define INSTALL_BUNDLE_ENABLE_ANALYTICS (!WITH_EDITOR)
#endif

enum class EPatchCheckResult : uint8;
class IAnalyticsProviderET;
class IInstallBundleSource;
namespace UE::IoStore { class IOnDemandIoStore; }
struct FPakMountOptions;

class FDefaultInstallBundleManager : public IInstallBundleManager
{
protected:
	// Strongly Typed enums do not work well for this thing's use case
	struct FContentRequestBatchNS
	{
		enum Enum : int
		{
			Requested,
			Cache,
			Install,
			Count,
		};
	};
	using EContentRequestBatch = FContentRequestBatchNS::Enum;
	friend struct NEnumRangePrivate::TEnumRangeTraits<EContentRequestBatch>;
	friend DEFAULTINSTALLBUNDLEMANAGER_API const TCHAR* LexToString(FDefaultInstallBundleManager::EContentRequestBatch Val);

	// Strongly Typed enums do not work well for this thing's use case
	struct FContentReleaseRequestBatchNS
	{
		enum Enum : int
		{
			Requested,
			Release,
			Count,
		};
	};
	using EContentReleaseRequestBatch = FContentReleaseRequestBatchNS::Enum;
	friend struct NEnumRangePrivate::TEnumRangeTraits<EContentReleaseRequestBatch>;
	friend DEFAULTINSTALLBUNDLEMANAGER_API const TCHAR* LexToString(FDefaultInstallBundleManager::EContentReleaseRequestBatch Val);
	
	enum class EBundleState : int
	{
		NotInstalled,
		NeedsUpdate,
		NeedsMount,
		Mounted,
		Count,
	};
	friend const TCHAR* LexToString(FDefaultInstallBundleManager::EBundleState Val)
	{
		static const TCHAR* Strings[] =
		{
			TEXT("NotInstalled"),
			TEXT("NeedsUpdate"),
			TEXT("NeedsMount"),
			TEXT("Mounted"),
		};

		return InstallBundleUtil::TLexToString(Val, Strings);
	}
	bool StateSignifiesNeedsInstall(EBundleState StateIn)
	{
		return (StateIn == EBundleState::NotInstalled || StateIn == EBundleState::NeedsUpdate);
	}

	enum class EAsyncInitStep : int
	{
		None,
		InitBundleSources,
		InitBundleCaches,
		InitPersisentReportCache,
		QueryBundleInfo,
		SetUpdateBundleInfoCallback,
		CreateAnalyticsSession,
		Finishing,
		Count,
	};

	friend const TCHAR* LexToString(FDefaultInstallBundleManager::EAsyncInitStep Val)
	{
		static const TCHAR* Strings[] =
		{
			TEXT("None"),
			TEXT("InitBundleSources"),
			TEXT("InitBundleCaches"),
			TEXT("InitPersisentReportCache"),
			TEXT("QueryBundleInfo"),
			TEXT("SetUpdateBundleInfoCallback"),
			TEXT("CreateAnalyticsSession"),
			TEXT("Finishing"),
		};

		return InstallBundleUtil::TLexToString(Val, Strings);
	}

	enum class EAsyncInitStepResult : int
	{
		Waiting,
		Done,
	};
	
	enum class EBundlePrereqs : int
	{
		CacheHintRequested,
		RequiresLatestClient,
		HasNoPendingCancels,
		HasNoPendingReleaseRequests,
		HasNoPendingUpdateRequests,
		DetermineSteps,
		Count
	};

	friend const TCHAR* LexToString(EBundlePrereqs Val)
	{
		static const TCHAR* Strings[] =
		{
			TEXT("CacheHintRequested"),
			TEXT("RequiresLatestClient"),
			TEXT("HasNoPendingCancels"),
			TEXT("HasNoPendingReleaseRequests"),
			TEXT("HasNoPendingUpdateRequests"),
			TEXT("DetermineSteps"),
		};

		return InstallBundleUtil::TLexToString(Val, Strings);
	}

	struct FBundleSourceRelevance
	{
		FInstallBundleSourceType SourceType;
		bool bIsRelevant = true;

		bool operator==(const FBundleSourceRelevance& Other) const { return SourceType == Other.SourceType; }
	};

	struct FBundleContentPaths
	{
		TArray<TPair<FString, FPakMountOptions>> ContentPaths;
		TArray<FString> AdditionalRootDirs;
		FString ProjectName;
		int32 PakOrder = INDEX_NONE;
		bool bContainsChunks = false;
	};

	friend struct FBundleInfo;
	struct FBundleInfo
	{
	public:
		EBundleState GetBundleStatus(const FDefaultInstallBundleManager& BundleMan) const
		{
			check(FPlatformTLS::GetCurrentThreadId() == GGameThreadId);
			check(BundleMan.InitState == EInstallBundleManagerInitState::Succeeded || BundleMan.bIsCurrentlyInAsyncInit);
			return BundleStatus;
		}
		void SetBundleStatus(const FDefaultInstallBundleManager& BundleMan, EBundleState InBundleState)
		{
			check(FPlatformTLS::GetCurrentThreadId() == GGameThreadId);
			check(BundleMan.InitState == EInstallBundleManagerInitState::Succeeded || BundleMan.bIsCurrentlyInAsyncInit);
			BundleStatus = InBundleState;
		}
		bool GetMustWaitForPSOCache(const FDefaultInstallBundleManager& BundleMan) const
		{
			check(FPlatformTLS::GetCurrentThreadId() == GGameThreadId);
			check(BundleMan.InitState == EInstallBundleManagerInitState::Succeeded);
			return bWaitForPSOCache;
		}
		uint32 GetInitialShaderPrecompiles(const FDefaultInstallBundleManager& BundleMan) const
		{
			check(FPlatformTLS::GetCurrentThreadId() == GGameThreadId);
			check(BundleMan.InitState == EInstallBundleManagerInitState::Succeeded);
			return InitialShaderPrecompiles;
		}
		void SetMustWaitForPSOCache(const FDefaultInstallBundleManager& BundleMan, uint32 InNumPSOPrecompilesRemaining)
		{
			check(FPlatformTLS::GetCurrentThreadId() == GGameThreadId);
			check(BundleMan.InitState == EInstallBundleManagerInitState::Succeeded);
			bWaitForPSOCache = InNumPSOPrecompilesRemaining > 0;
			if (InNumPSOPrecompilesRemaining > InitialShaderPrecompiles)
			{
				InitialShaderPrecompiles = InNumPSOPrecompilesRemaining;
			}
		}
	private:
		EBundleState BundleStatus = EBundleState::NotInstalled;
		uint32 InitialShaderPrecompiles = 0;
	public:
		FString BundleNameString; // Since FNames do not preserve casing
		TArray<EBundlePrereqs> Prereqs;
		TArray<FBundleSourceRelevance> ContributingSources; // Sources contributing to this bundle info		
		FBundleContentPaths ContentPaths; // Only valid if BundleStatus >= NeedsMount
		EInstallBundlePriority Priority = EInstallBundlePriority::Low;
	private:
		bool bWaitForPSOCache = false;
	public:
		bool bReleaseRequired = false; // A bundle source may have done install work so expects a release call
		bool bIsStartup = false;
		bool bContainsIoStoreOnDemandTocs = false;
		bool bMountedOnDemandTocs = false;
	};

	// GetBundleStatus protects erroneous accesses of the bundle status before initialization is complete by throwing an assert 
	UE_API EBundleState GetBundleStatus(const FBundleInfo& BundleInfo) const;
	UE_API void SetBundleStatus(FBundleInfo& BundleInfo, EBundleState InBundleState);
	UE_API bool GetMustWaitForPSOCache(const FBundleInfo& BundleInfo) const;
	UE_API uint32 GetInitialShaderPrecompiles(const FBundleInfo& BundleInfo) const;
	UE_API void SetMustWaitForPSOCache(FBundleInfo& BundleInfo, uint32 InNumPSOPrecompilesRemaining);

	struct FGetContentStateRequest
	{
		TMap<FInstallBundleSourceType, FInstallBundleCombinedContentState> BundleSourceContentStates;

		TArray<FName> BundleNames;

		EInstallBundleGetContentStateFlags Flags = EInstallBundleGetContentStateFlags::None;

		bool Started = false;
		bool bCancelled = false;

		//Used to track an individual request so that it can be canceled
		FName RequestTag;

		void SetCallback(FInstallBundleGetContentStateDelegate NewCallback)
		{
			Callback = MoveTemp(NewCallback);
		}

		void ExecCallbackIfValid(FInstallBundleCombinedContentState BundleState)
		{
			if (!bCancelled)
			{
				Callback.ExecuteIfBound(MoveTemp(BundleState));
			}
		}

		const FDelegateHandle GetCallbackDelegateHandle()
		{
			return Callback.GetHandle();
		}

	private:
		FInstallBundleGetContentStateDelegate Callback;
	};
	using FGetContentStateRequestRef = TSharedRef<FGetContentStateRequest>;
	using FGetContentStateRequestPtr = TSharedPtr<FGetContentStateRequest>;

	struct FGetInstallStateRequest
	{
		TArray<FName> BundleNames;

		bool bCancelled = false;

		//Used to track an individual request so that it can be canceled
		FName RequestTag;

		void SetCallback(FInstallBundleGetInstallStateDelegate NewCallback)
		{
			Callback = MoveTemp(NewCallback);
		}

		void ExecCallbackIfValid(FInstallBundleCombinedInstallState BundleState)
		{
			if (!bCancelled)
			{
				Callback.ExecuteIfBound(MoveTemp(BundleState));
			}
		}

		const FDelegateHandle GetCallbackDelegateHandle()
		{
			return Callback.GetHandle();
		}

	private:
		FInstallBundleGetInstallStateDelegate Callback;
	};
	using FGetInstallStateRequestRef = TSharedRef<FGetInstallStateRequest>;
	using FGetInstallStateRequestPtr = TSharedPtr<FGetInstallStateRequest>;

	enum class EContentRequestStepResult : int
	{
		Waiting,
		Done,
	};

	enum class EContentRequestState : int
	{
		ReservingCache,
		FinishingCache,
		UpdatingBundleSources,
		Mounting,
		WaitingForShaderCache,
		Finishing,
		CleaningUp,
		Count,
	};

	friend const TCHAR* LexToString(EContentRequestState Val)
	{
		static const TCHAR* Strings[] =
		{
			TEXT("ReservingCache"),
			TEXT("FinishingCache"),
			TEXT("UpdatingBundleSources"),
			TEXT("Mounting"),
			TEXT("WaitingForShaderCache"),
			TEXT("Finishing"),
			TEXT("CleaningUp"),
		};

		return InstallBundleUtil::TLexToString(Val, Strings);
	}

	enum class ECacheEvictionRequestorType : int
	{
		CacheFlush,
		ContentRequest
	};

	struct FCacheEvictionRequestor
	{
		TMap<FName, TArray<FInstallBundleSourceType>> BundlesToEvictFromSourcesMap;

		virtual ~FCacheEvictionRequestor() = default;

		virtual FString GetEvictionRequestorName() const = 0;
		virtual ECacheEvictionRequestorType GetEvictionRequestorType() const = 0;
		virtual ELogVerbosity::Type GetLogVerbosityOverride() const = 0;
	};
	using FCacheEvictionRequestorRef = TSharedRef<FCacheEvictionRequestor>;
	using FCacheEvictionRequestorPtr = TSharedPtr<FCacheEvictionRequestor>;
	using FCacheEvictionRequestorWeakPtr = TWeakPtr<FCacheEvictionRequestor>;

	struct FCacheFlushRequest : FCacheEvictionRequestor
	{
		FInstallBundleSourceOrCache SourceOrCache; // Bundles are evicted from all caches, but we gather them from only this one if set
		ELogVerbosity::Type LogVerbosityOverride = ELogVerbosity::NoLogging;

		FInstallBundleManagerFlushCacheCompleteDelegate Callback;

		virtual FString GetEvictionRequestorName() const override 
		{
			if (SourceOrCache.HasSubtype<FName>())
			{
				return FString::Printf(TEXT("CacheFlush(%s)"), *SourceOrCache.GetSubtype<FName>().ToString());
			}
			else if (SourceOrCache.HasSubtype<FInstallBundleSourceType>())
			{
				return FString::Printf(TEXT("CacheFlush(%s)"), LexToString(SourceOrCache.GetSubtype<FInstallBundleSourceType>()));
			}
			else
			{
				return TEXT("CacheFlush(All)");
			}
		}
		virtual ECacheEvictionRequestorType GetEvictionRequestorType() const override { return ECacheEvictionRequestorType::CacheFlush; }
		virtual ELogVerbosity::Type GetLogVerbosityOverride() const override { return LogVerbosityOverride; }
	};
	using FCacheFlushRequestRef = TSharedRef<FCacheFlushRequest>;
	using FCacheFlushRequestPtr = TSharedPtr<FCacheFlushRequest>;
	using FCacheFlushRequestWeakPtr = TWeakPtr<FCacheFlushRequest>;

	struct FContentRequest : FCacheEvictionRequestor
	{
		~FContentRequest();
		
		int32 iStep = INDEX_NONE;
		EContentRequestStepResult StepResult = EContentRequestStepResult::Done;
		TArray<EContentRequestState> Steps;
		TStaticArray<int32, EContentRequestBatch::Count> iOnCanceledStep{ InPlace, INDEX_NONE };

		int32 iPrereq = INDEX_NONE;
		TArray<EBundlePrereqs> Prereqs;
		FDelegateHandle CheckLastestClientDelegateHandle;

		EInstallBundleRequestFlags Flags = EInstallBundleRequestFlags::None;

		ELogVerbosity::Type LogVerbosityOverride = ELogVerbosity::NoLogging;

		bool bShouldSendAnalytics = true;
		bool bIsCanceled = false;
		bool bFinishWhenCanceled = true;  // Whether to run cleanup and callback when canceled
		bool bDidCacheHintRequested = false; // Whether this request hinted to the bundle caches that the bundle was requested
		bool bContentWasInstalled = false;
		FName BundleName;

		EInstallBundleResult Result = EInstallBundleResult::OK;

		EInstallBundlePauseFlags LastSentPauseFlags = EInstallBundlePauseFlags::None;
		TMap<FInstallBundleSourceType, EInstallBundlePauseFlags> SourcePauseFlags;
		bool bForcePauseCallback = false;

		EInstallBundleCacheReserveResult LastCacheReserveResult = EInstallBundleCacheReserveResult::Success;

		// how many results we are expected to have in the SourceRequestResults array
		int RequiredSourceRequestResultsCount = 0;
		// completion results from each bundle source
		TMap<FInstallBundleSourceType, FInstallBundleSourceUpdateContentResultInfo> SourceRequestResults;

		TArray<TUniquePtr<UE::IoStore::FOnDemandMountArgs>> OnDemandMountArgs;

		FText OptionalErrorText;
		FString OptionalErrorCode;

		TMap<FInstallBundleSourceType, FInstallBundleSourceProgress> CachedSourceProgress;

		InstallBundleUtil::FContentRequestSharedContextPtr RequestSharedContext;

		// If needed, Keep the engine awake while process requests
		TOptional<InstallBundleUtil::FInstallBundleManagerKeepAwake> KeepAwake;

		// If needed, Banish screen savers
		TOptional<InstallBundleUtil::FInstallBundleManagerScreenSaverControl> ScreenSaveControl;

		virtual FString GetEvictionRequestorName() const override { return BundleName.ToString(); }
		virtual ECacheEvictionRequestorType GetEvictionRequestorType() const override { return ECacheEvictionRequestorType::ContentRequest; }
		virtual ELogVerbosity::Type GetLogVerbosityOverride() const override { return LogVerbosityOverride;  }

		void* GetStatsCookie() const { return const_cast<FContentRequest*>(this); }
		InstallBundleUtil::FContentRequestStatsKey GetStatsKey() const { return InstallBundleUtil::FContentRequestStatsKey(BundleName, GetStatsCookie()); }
	};
	using FContentRequestRef = TSharedRef<FContentRequest>;
	using FContentRequestPtr = TSharedPtr<FContentRequest>;
	using FContentRequestWeakPtr = TWeakPtr<FContentRequest>;

	enum class EContentReleaseRequestState : int
	{
		Unmounting,
		UpdatingBundleSources,
		Finishing,
		CleaningUp,
		Count
	};
	
	friend const TCHAR* LexToString(EContentReleaseRequestState Val)
	{
		static const TCHAR* Strings[] =
		{
			TEXT("Unmounting"),
			TEXT("UpdatingBundleSources"),
			TEXT("Finishing"),
			TEXT("CleaningUp"),
		};

		return InstallBundleUtil::TLexToString(Val, Strings);
	}

	struct FContentReleaseRequest
	{
		EContentRequestStepResult StepResult = EContentRequestStepResult::Done;
		TArray<EContentReleaseRequestState> Steps;
		int32 iStep = INDEX_NONE;
		TStaticArray<int32, EContentReleaseRequestBatch::Count> iOnCanceledStep{ InPlace, INDEX_NONE };

		TArray<EBundlePrereqs> Prereqs;
		int32 iPrereq = INDEX_NONE;
		
		EInstallBundleReleaseRequestFlags Flags = EInstallBundleReleaseRequestFlags::None;

		EInstallBundleReleaseResult Result = EInstallBundleReleaseResult::OK;
		
		FName BundleName;

		TMap<FInstallBundleSourceType, TOptional<FInstallBundleSourceReleaseContentResultInfo>> SourceReleaseRequestResults;
		TMap<FInstallBundleSourceType, TOptional<FInstallBundleSourceReleaseContentResultInfo>> SourceRemoveRequestResults;

		ELogVerbosity::Type LogVerbosityOverride = ELogVerbosity::NoLogging;

		bool bIsCanceled = false;
		bool bFinishWhenCanceled = true;  // Whether to run cleanup and callback when canceled
	};
	using FContentReleaseRequestRef = TSharedRef<FContentReleaseRequest>;
	using FContentReleaseRequestPtr = TSharedPtr<FContentReleaseRequest>;
	using FContentReleaseRequestWeakPtr = TWeakPtr<FContentReleaseRequest>;

	struct FContentPatchCheckSharedContext
	{
		TMap<FInstallBundleSourceType, bool> Results;
	};
	using FContentPatchCheckSharedContextRef = TSharedRef<FContentPatchCheckSharedContext>;

public:
	typedef TUniqueFunction<TSharedPtr<IInstallBundleSource>(FInstallBundleSourceType)> FInstallBundleSourceFactoryFunction;

	UE_DEPRECATED(5.4, "GInstallBundleManagerIni is deprecated, use InstallBundle.ini hierarchy instead.")
	FDefaultInstallBundleManager(const TCHAR* InConfigBaseName, FInstallBundleSourceFactoryFunction InBundleSourceFactory = nullptr)
		: FDefaultInstallBundleManager(MoveTemp(InBundleSourceFactory))
	{}

	UE_API FDefaultInstallBundleManager(FInstallBundleSourceFactoryFunction InBundleSourceFactory = nullptr);
	FDefaultInstallBundleManager(const FDefaultInstallBundleManager& Other) = delete;
	FDefaultInstallBundleManager& operator=(const FDefaultInstallBundleManager& Other) = delete;

	UE_API virtual ~FDefaultInstallBundleManager();

	UE_API virtual void Initialize() override;

	//Tick
protected:
	UE_API bool Tick(float dt);

	UE_API EInstallBundleManagerInitErrorHandlerResult HandleAsyncInitError(EInstallBundleManagerInitResult InitResultError);

	UE_API void TickInit();

	UE_API void TickGetContentState();

	UE_API void TickGetInstallState();

	UE_API FInstallBundleCombinedInstallState GetInstallStateInternal(TArrayView<const FName> BundleNames) const;

	UE_API void CacheHintRequested(FContentRequestRef Request, bool bRequested);

	UE_API void CheckPrereqHasNoPendingCancels(FContentRequestRef Request);

	UE_API void CheckPrereqHasNoPendingCancels(FContentReleaseRequestRef Request);

	UE_API void CheckPrereqHasNoPendingReleaseRequests(FContentRequestRef Request);

	UE_API void CheckPrereqHasNoPendingUpdateRequests(FContentReleaseRequestRef Request);

	UE_API void CheckPrereqLatestClient(FContentRequestRef Request);

	UE_API void HandlePatchInformationReceived(EInstallBundleManagerPatchCheckResult Result, FContentRequestRef Request);

	UE_API void DetermineSteps(FContentRequestRef Request);

	UE_API void DetermineSteps(FContentReleaseRequestRef Request);

	UE_API void AddRequestToInitialBatch(FContentRequestRef Request);

	UE_API void AddRequestToInitialBatch(FContentReleaseRequestRef Request);

	UE_API void ReserveCache(FContentRequestRef Request);

	UE_API void TryReserveCache(FContentRequestRef Request);

	UE_API void RequestEviction(FCacheEvictionRequestorRef Requestor);

	UE_API void CacheEvictionComplete(TSharedRef<IInstallBundleSource> Source, FInstallBundleSourceReleaseContentResultInfo InResultInfo);
	UE_API void CacheEvictionComplete(TSharedRef<IInstallBundleSource> Source, const FInstallBundleSourceReleaseContentResultInfo& InResultInfo, FCacheEvictionRequestorRef Requestor);

	UE_API void UpdateBundleSources(FContentRequestRef Request);

	UE_API void UpdateBundleSourceComplete(TSharedRef<IInstallBundleSource> Source, FInstallBundleSourceUpdateContentResultInfo InResultInfo, FContentRequestRef Request);

	UE_API void UpdateBundleSourcePause(TSharedRef<IInstallBundleSource> Source, FInstallBundleSourcePauseInfo InPauseInfo, FContentRequestRef Request);

	UE_API void UpdateBundleSources(FContentReleaseRequestRef Request);

	UE_API void UpdateBundleSourceReleaseComplete(TSharedRef<IInstallBundleSource> Source, FInstallBundleSourceReleaseContentResultInfo InResultInfo, FContentReleaseRequestRef Request);

	UE_API void MountPaks(FContentRequestRef Request);

	static UE_API bool MountPaksInList(TArrayView<TPair<FString, FPakMountOptions>> Paths, TValueOrError<void, UE::UnifiedError::FError>& OutResult, ELogVerbosity::Type LogVerbosityOverride, const int32 PakOrder = INDEX_NONE);

	UE_API void UnmountPaks(FContentReleaseRequestRef Request);

	UE_API virtual bool AllowIoStoreOnDemandMount(FContentRequestRef Request, const FBundleInfo& BundleInfo);

	UE_API virtual TArray<TPair<FString, FPakMountOptions>> GetPakMountList(FContentRequestRef Request, const FBundleInfo& BundleInfo);

	virtual void OnPaksMountingInternal(FContentRequestRef Request, FBundleInfo& BundleInfo) {}
	virtual void OnPaksMountedInternal(FContentRequestRef Request, FBundleInfo& BundleInfo) {}
	virtual void OnPaksUnmountedInternal(FContentReleaseRequestRef Request, FBundleInfo& BundleInfo) {};

	UE_API void WaitForShaderCache(FContentRequestRef Request);

	UE_API void FinishRequest(FContentRequestRef Request);

	UE_API void FinishRequest(FContentReleaseRequestRef Request);

	UE_API void TickUpdatePrereqs();

	UE_API void TickReleasePrereqs();

	UE_API void TickContentRequests();

	UE_API void TickReserveCache();

	UE_API void TickCacheFlush();

	UE_API void TickWaitForShaderCache();

	UE_API void TickPauseStatus(bool bForceCallback);

	UE_API void TickAsyncIOTasks();

	UE_API void TickReleaseRequests();

	UE_API void TickPruneBundleInfo();

	UE_API bool TickReporting(float dt);

	UE_API void IterateContentRequests(TFunctionRef<bool(const FContentRequestRef& QueuedRequest)> OnFound);
	UE_API void IterateReleaseRequests(TFunctionRef<bool(const FContentReleaseRequestRef& QueuedRequest)> OnFound);

	UE_API void IterateContentRequestsForBundle(FName BundleName, TFunctionRef<bool(const FContentRequestRef& QueuedRequest)> OnFound);
	UE_API void IterateReleaseRequestsForBundle(FName BundleName, TFunctionRef<bool(const FContentReleaseRequestRef& QueuedRequest)> OnFound);

	UE_API TSet<FName> GetBundleDependencies(FName InBundleName, bool* bSkippedUnknownBundles = nullptr) const;

	UE_API TSet<FName> GetBundleDependencies(TArrayView<const FName> InBundleNames, bool* bSkippedUnknownBundles = nullptr) const;

	UE_API TSet<FName> GatherBundlesForRequest(TArrayView<const FName> InBundleNames, EInstallBundleRequestInfoFlags& OutFlags);

	UE_API TSet<FName> GatherBundlesForRequest(TArrayView<const FName> InBundleNames);

	UE_API FInstallBundleSourceType GetBundleSourceFallback(FInstallBundleSourceType Type) const;

	UE_API EInstallBundleSourceUpdateBundleInfoResult OnUpdateBundleInfoFromSource(TSharedRef<IInstallBundleSource> Source, FInstallBundleSourceUpdateBundleInfoResult Result);
	UE_API void OnBundleLostRelevanceForSource(TSharedRef<IInstallBundleSource> Source, TSet<FName> BundleNames);

	UE_API void StartClientPatchCheck();
	UE_API void StartContentPatchCheck();
	UE_API void HandleClientPatchCheck(EPatchCheckResult Result);
	UE_API void HandleBundleSourceContentPatchCheck(TSharedRef<IInstallBundleSource> Source, bool bContentPatchRequired, FContentPatchCheckSharedContextRef Context);
	UE_API void HandleContentPatchCheck(FContentPatchCheckSharedContextRef Context);

	UE_API bool CancelUpdateContentInternal(TArrayView<const FName> BundleNames);
	UE_API bool CancelReleaseContentInternal(TArrayView<const FName> BundleNames);
	
	void OnChunkDownloadMetrics(FInstallBundleChunkDownloadInfo ChunkDownloadInfo);

	// IInstallBundleManager interface
public:
	UE_API virtual bool HasBundleSource(FInstallBundleSourceType SourceType) const override;

	UE_API virtual const TSharedPtr<IInstallBundleSource> GetBundleSource(FInstallBundleSourceType SourceType) const override;

	UE_API virtual void SetContext(FName ContextName) override;
	UE_API virtual bool CopyReportTo(FInstallManagerBundleReport& Report) override;

	UE_API virtual FDelegateHandle PushInitErrorCallback(FInstallBundleManagerInitErrorHandler Callback) override;
	UE_API virtual void PopInitErrorCallback() override;
	UE_API virtual void PopInitErrorCallback(FDelegateHandle Handle) override;
	UE_API virtual void PopInitErrorCallback(FDelegateUserObjectConst InUserObject) override;

	UE_API virtual EInstallBundleManagerInitState GetInitState() const override;

	UE_API virtual TValueOrError<FInstallBundleRequestInfo, EInstallBundleResult> RequestUpdateContent(TArrayView<const FName> InBundleNames, EInstallBundleRequestFlags Flags, ELogVerbosity::Type LogVerbosityOverride = ELogVerbosity::NoLogging, InstallBundleUtil::FContentRequestSharedContextPtr RequestSharedContext = nullptr) override;

	UE_API virtual FDelegateHandle GetContentState(TArrayView<const FName> InBundleNames, EInstallBundleGetContentStateFlags Flags, bool bAddDependencies, FInstallBundleGetContentStateDelegate Callback, FName RequestTag = TEXT("None")) override;

	UE_API virtual void CancelAllGetContentStateRequestsForTag(FName RequestTag) override;
	UE_API virtual void CancelAllGetContentStateRequests(FDelegateHandle Handle) override;

	UE_API virtual FDelegateHandle GetInstallState(TArrayView<const FName> BundleNames, bool bAddDependencies, FInstallBundleGetInstallStateDelegate Callback, FName RequestTag = NAME_None) override;

	UE_API virtual TValueOrError<FInstallBundleCombinedInstallState, EInstallBundleResult> GetInstallStateSynchronous(TArrayView<const FName> BundleNames, bool bAddDependencies) const override;
	UE_API virtual EInstallBundleResult GetAllMountedBundles(TArray<FName>& OutBundleNames) const override;
	UE_API virtual EInstallBundleResult GetAllBundlesWithState(EInstallBundleInstallState InstallState, TArray<FName>& OutBundleNames) const override;

	UE_API virtual void CancelAllGetInstallStateRequestsForTag(FName RequestTag) override;
	UE_API virtual void CancelAllGetInstallStateRequests(FDelegateHandle Handle) override;

	UE_API virtual TValueOrError<FInstallBundleReleaseRequestInfo, EInstallBundleResult> RequestReleaseContent(TArrayView<const FName> ReleaseNames, EInstallBundleReleaseRequestFlags Flags, TArrayView<const FName> KeepNames = TArrayView<const FName>(), ELogVerbosity::Type LogVerbosityOverride = ELogVerbosity::NoLogging) override;
	
	UE_API virtual EInstallBundleResult FlushCache(FInstallBundleSourceOrCache SourceOrCache, FInstallBundleManagerFlushCacheCompleteDelegate Callback, ELogVerbosity::Type LogVerbosityOverride = ELogVerbosity::NoLogging) override;

	UE_API virtual TArray<FInstallBundleCacheStats> GetCacheStats(EInstallBundleCacheStatsFlags Flags = EInstallBundleCacheStatsFlags::None, ELogVerbosity::Type LogVerbosityOverride = ELogVerbosity::NoLogging) override;
	UE_API virtual TOptional<FInstallBundleCacheStats> GetCacheStats(FInstallBundleSourceOrCache SourceOrCache, EInstallBundleCacheStatsFlags Flags = EInstallBundleCacheStatsFlags::None, ELogVerbosity::Type LogVerbosityOverride = ELogVerbosity::NoLogging) override;

	UE_API virtual void RequestRemoveContentOnNextInit(TArrayView<const FName> RemoveNames, TArrayView<const FName> KeepNames = TArrayView<const FName>()) override;

	UE_API virtual void CancelRequestRemoveContentOnNextInit(TArrayView<const FName> BundleName) override;

	UE_API virtual TArray<FName> GetRequestedRemoveContentOnNextInit() const override;

	UE_API virtual void CancelUpdateContent(TArrayView<const FName> BundleNames) override;

	UE_API virtual void PauseUpdateContent(TArrayView<const FName> BundleNames) override;

	UE_API virtual void ResumeUpdateContent(TArrayView<const FName> BundleNames) override;

	UE_API virtual void RequestPausedBundleCallback() override;

	UE_API virtual TOptional<FInstallBundleProgress> GetBundleProgress(FName BundleName) const override;

	UE_API virtual EInstallBundleRequestFlags GetModifyableContentRequestFlags() const override;
	UE_API virtual void UpdateContentRequestFlags(TArrayView<const FName> BundleNames, EInstallBundleRequestFlags AddFlags, EInstallBundleRequestFlags RemoveFlags) override;
	UE_API virtual void SetCellularPreference(int32 Value) override;
	UE_API virtual void SetLimitedDataPreference(int32 Value) override;

	UE_API virtual void SetCacheSize(FName CacheName, uint64 CacheSize) override;

	UE_API virtual void StartPatchCheck() override;
	UE_API virtual void AddEnvironmentWantsPatchCheckBackCompatDelegate(FName Tag, FInstallBundleManagerEnvironmentWantsPatchCheck Delegate) override;
	UE_API virtual void RemoveEnvironmentWantsPatchCheckBackCompatDelegate(FName Tag) override;
	UE_API virtual bool SupportsEarlyStartupPatching() const override;

	UE_API virtual bool IsNullInterface() const override;

	UE_API virtual void SetErrorSimulationCommands(const FString& CommandLine) override;

	//For overrides that we need to handle even when in a shipping build
	UE_API void SetCommandLineOverrides(const FString& CommandLine);

	UE_API virtual TSharedPtr<IAnalyticsProviderET> GetAnalyticsProvider() const override;

	UE_API virtual void StartSessionPersistentStatTracking(const FString& SessionName, const TArray<FName>& RequiredBundles = TArray<FName>(), const FString& ExpectedAnalyticsID = FString(), bool bForceResetStatData = false, const FInstallBundleCombinedContentState* State = nullptr) override;
	UE_API virtual void StopSessionPersistentStatTracking(const FString& SessionName) override;

#if !UE_BUILD_SHIPPING
	UE_API virtual void GetDebugText(TArray<FString>& Output) override;
#endif

	virtual bool HasEverUpdatedContent() const { return bHasEverUpdatedContent; }

protected:
	//Special version of these to wrap our calls to PersistentStats
	UE_API void StartBundlePersistentStatTracking(TSharedRef<FContentRequest> ContentRequest, const FString& ExpectedAnalyticsID = FString(), bool bForceResetStatData = false);
	UE_API void StopBundlePersistentStatTracking(TSharedRef<FContentRequest> ContentRequest);
	UE_API void PersistentTimingStatsBegin(TSharedRef<FContentRequest> ContentRequest, InstallBundleUtil::PersistentStats::ETimingStatNames TimerStatName);
	UE_API void PersistentTimingStatsEnd(TSharedRef<FContentRequest> ContentRequest, InstallBundleUtil::PersistentStats::ETimingStatNames TimerStatName);
	
	UE_API TArray<TSharedPtr<IInstallBundleSource>> GetEnabledBundleSourcesForRequest(FContentRequestRef Request) const;
	UE_API virtual TArray<TSharedPtr<IInstallBundleSource>> GetEnabledBundleSourcesForRequest(const FBundleInfo& BundleInfo) const;

	// Initialization state machine
protected:
	UE_API EInstallBundleManagerInitResult Init_DefaultBundleSources();
	UE_API EInstallBundleManagerInitResult Init_TryCreateBundleSources(TArray<FInstallBundleSourceType> SourcesToCreate, TArray<TSharedPtr<IInstallBundleSource>>* OutNewSources = nullptr);

	UE_API FInstallBundleSourceType FindFallbackSource(FInstallBundleSourceType SourceType);
	UE_API void AsyncInit_InitBundleSources();
	UE_API void AsyncInit_OnBundleSourceInitComplete(TSharedRef<IInstallBundleSource> Source, FInstallBundleSourceAsyncInitInfo InInitInfo);
	UE_API void AsyncInit_InitBundleCaches();
	UE_API void AsyncInit_InitPersistentReportCache();
	UE_API void AsyncInit_QueryBundleInfo();
	UE_API void AsyncInit_OnQueryBundleInfoComplete(TSharedRef<IInstallBundleSource> Source, FInstallBundleSourceBundleInfoQueryResult Result);
	UE_API void AsyncInit_OnQueryBundleInfoComplete_HandleClientPatchCheck(EPatchCheckResult Result);
	UE_API void AsyncInit_SetUpdateBundleInfoCallback();
	UE_API void AsyncInit_CreateAnalyticsSession();
	UE_API void AsyncInit_FireInitAnlaytic(bool bCanRetry);

	UE_API void StatsBegin(const InstallBundleUtil::FContentRequestStatsKey& Key);
	UE_API void StatsEnd(const InstallBundleUtil::FContentRequestStatsKey& Key);
	UE_API void StatsBegin(const InstallBundleUtil::FContentRequestStatsKey& Key, EContentRequestState State);
	UE_API void StatsEnd(const InstallBundleUtil::FContentRequestStatsKey& Key, EContentRequestState State, uint64 DataSize = 0);
	UE_API void LogStats(const InstallBundleUtil::FContentRequestStatsKey& Key, ELogVerbosity::Type LogVerbosityOverride);

protected:
	FTSTicker::FDelegateHandle TickHandle;
	FTSTicker::FDelegateHandle ReportingHandle;
	FDelegateHandle AsyncInit_PatchCheckHandle;
	FDelegateHandle PatchCheckHandle;
	UE::IoStore::IOnDemandIoStore* OnDemandIoStore = nullptr;
	FInstallBundleSourceFactoryFunction InstallBundleSourceFactory;

	TMap<FName, FBundleInfo> BundleInfoMap;
	TSet<FName> BundlesInfosToPrune;

	TMap<FInstallBundleSourceType, TSharedPtr<IInstallBundleSource>> BundleSources;
	TMap<FInstallBundleSourceType, FInstallBundleSourceType> BundleSourceFallbacks;

	TMap<FName, TSharedRef<FInstallBundleCache>> BundleCaches;
	TMap<FInstallBundleSourceType, FName> BundleSourceCaches;
	TMap<FName, uint64> BundleCacheSizeOverrides;

	TMap<TTuple<FInstallBundleSourceType, FName>, TArray<FCacheEvictionRequestorRef>> PendingCacheEvictions; // (Source, Bundle) -> List of requestors
	TMap<TTuple<FName, FName>, TArray<FInstallBundleSourceType>> CachesPendingEvictToSources; // (Cache, Bundle) -> List of Sources

	// Only used during Init
	TMap<FInstallBundleSourceType, TOptional<FInstallBundleSourceAsyncInitInfo>> BundleSourceInitResults;
	TMap<FInstallBundleSourceType, FInstallBundleSourceBundleInfoQueryResult> BundleSourceBundleInfoQueryResults;

	// Init
	EInstallBundleManagerInitState InitState = EInstallBundleManagerInitState::NotInitialized;
	EInstallBundleManagerInitResult InitResult = EInstallBundleManagerInitResult::OK;
	TArray<FInstallBundleManagerInitErrorHandler> InitErrorHandlerStack;
	TArray<TSharedPtr<IInstallBundleSource>> BundleSourcesToDelete;
	EAsyncInitStep InitStep = EAsyncInitStep::None;
	EAsyncInitStep LastInitStep = EAsyncInitStep::None;
	EAsyncInitStepResult InitStepResult = EAsyncInitStepResult::Done;
	bool bUnrecoverableInitError = false;
	bool bIsCurrentlyInAsyncInit = false;
	double LastInitRetryTimeSeconds = 0.0;
	double InitRetryTimeDeltaSeconds = 0.0;

	// Content State Requests
	TArray<FGetContentStateRequestRef> GetContentStateRequests;
	TArray<FGetInstallStateRequestRef> GetInstallStateRequests;

	// Content Requests
	TArray<FContentRequestRef> ContentRequests[EContentRequestBatch::Count];

	// Content Release Requests
	TArray<FContentReleaseRequestRef> ContentReleaseRequests[EContentReleaseRequestBatch::Count];
	
	// Cache Flush Requests
	TArray<FCacheFlushRequestRef> CacheFlushRequests;

	TSharedRef<InstallBundleManagerUtil::FPersistentStatContainer> PersistentStats;

	TArray<TUniquePtr<InstallBundleUtil::FInstallBundleTask>> AsyncIOTasks;
	
	bool bIsCheckingForPatch = false;
	bool bDelayCheckingForContentPatch = false;

#if INSTALL_BUNDLE_ALLOW_ERROR_SIMULATION
	// Error Simulation
	bool bSimulateClientNotLatest = false;
	bool bSimulateContentNotLatest = false;
#endif // INSTALL_BUNDLE_ALLOW_ERROR_SIMULATION

	//Not included in INSTALL_BUNDLE_ALLOW_ERROR_SIMULATION as we want to provide
	//this functionality even on ship builds
	bool bOverrideCommand_SkipPatchCheck = false;

	bool bHasEverUpdatedContent = false;

	// Persistent update state
	bool bPersistentStateDirty = false;
	FName ContextName;
	FInstallManagerBundleReport PersistentState;
	FInstallManagerBundleReportCache PersistentStateCache; // must be declared after AsyncIOTasks

protected:
	// Analytics
	TSharedPtr<IAnalyticsProviderET> AnalyticsProvider;	// use SetAnalyticsProvider() to update!
	TSharedRef<InstallBundleUtil::FContentRequestStatsMap> StatsMap;
	TMap<FName, FBundleRequestCompleteInfo> BundleRequestCompleteInfoMap;	// BundleName as key
	TMap<FName, TArray<FInstallBundleChunkDownloadInfo> > ChunkDownloadInfoMap;	// BundleName as key

	// Only to be called on the game thread, updates already initialized bundles with the new analytics provider,
	// and sets the new provider internally.
	UE_API void SetAnalyticsProvider(TSharedPtr<IAnalyticsProviderET> NewAnalyticsProvider);
};


ENUM_RANGE_BY_COUNT(FDefaultInstallBundleManager::EContentRequestBatch, FDefaultInstallBundleManager::EContentRequestBatch::Count);

DEFAULTINSTALLBUNDLEMANAGER_API const TCHAR* LexToString(FDefaultInstallBundleManager::EContentRequestBatch Val);


ENUM_RANGE_BY_COUNT(FDefaultInstallBundleManager::EContentReleaseRequestBatch, FDefaultInstallBundleManager::EContentReleaseRequestBatch::Count);

DEFAULTINSTALLBUNDLEMANAGER_API const TCHAR* LexToString(FDefaultInstallBundleManager::EContentReleaseRequestBatch Val);

#undef UE_API
