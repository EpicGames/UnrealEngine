// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "InstallBundleSourceInterface.h"
#include "InstallBundleUtils.h"

#define UE_API DEFAULTINSTALLBUNDLEMANAGER_API

namespace UE::IoStore{ class FOnDemandHostGroup; }

class FInstallBundleSourceBulk : public IInstallBundleSource
{
public:
	UE_API FInstallBundleSourceBulk();
	FInstallBundleSourceBulk(const FInstallBundleSourceBulk& Other) = delete;
	FInstallBundleSourceBulk& operator=(const FInstallBundleSourceBulk& Other) = delete;
	UE_API virtual ~FInstallBundleSourceBulk();

private:
	UE_API bool Tick(float dt);

	UE_API void TickInit();

	// Init
	UE_API void AsyncInit_FireInitAnlaytic();
	UE_API void AsyncInit_MakeBundlesForBulkBuild();

protected:
	UE_API virtual EInstallBundleInstallState GetBundleInstallState(FName BundleName);

	// IInstallBundleSource Interface
public:
	UE_API virtual FInstallBundleSourceType GetSourceType() const override;
	virtual float GetSourceWeight() const override { return 0.1f; }  // Low weight since all this source does in mount

	UE_API virtual FInstallBundleSourceInitInfo Init(
		TSharedRef<InstallBundleUtil::FContentRequestStatsMap> InRequestStats,
		TSharedPtr<IAnalyticsProviderET> AnalyticsProvider,
		TSharedPtr<InstallBundleUtil::PersistentStats::FPersistentStatContainerBase> PersistentStatsContainer) override;
	UE_API virtual void AsyncInit(FInstallBundleSourceInitDelegate Callback) override;

	UE_API virtual void AsyncInit_QueryBundleInfo(FInstallBundleSourceQueryBundleInfoDelegate Callback) override;

	UE_API virtual EInstallBundleManagerInitState GetInitState() const override;

	UE_API virtual FString GetContentVersion() const override;

	UE_API virtual TSet<FName> GetBundleDependencies(FName InBundleName, TSet<FName>* SkippedUnknownBundles /*= nullptr*/) const override;

	UE_API virtual void GetContentState(TArrayView<const FName> BundleNames, EInstallBundleGetContentStateFlags Flags, FInstallBundleGetContentStateDelegate Callback) override;

	UE_API virtual void RequestUpdateContent(FRequestUpdateContentBundleContext BundleContext) override;

	UE_API virtual void SetErrorSimulationCommands(const FString& CommandLine) override;

	virtual void SetAnalyticsProvider(TSharedPtr<IAnalyticsProviderET> NewAnalyticsProvider) override
	{
		AnalyticsProvider = NewAnalyticsProvider;
	}

	//Function that loads BulkBundleBuild information from a generated BulkBuildBundleIni instead of applying reg-ex at runtime
	//Returns true if BulkBundleBuild.ini existed and was parsed successfully, false otherwise
	//Removes any loaded entries from InOutFileList
	static UE_API bool TryLoadBulkBuildBundleMetadata(TArray<FString>& InOutFileList, TMap<FName, TArray<FString>>& InOutBulkBuildBundles);

	//Serialize out our BulkBundleBuild information to a BulkBundleBuild.ini file for future runs to not have to parse this information
	static UE_API void SerializeBulkBuildBundleMetadata(const TMap<FName, TArray<FString>>& BulkBuildBundles);

	// Support for on demand tocs, must be implemented by the game
protected:

	// Optionally used to modify what ContentPaths are returned in the base implementation of RequestUpdateContent
	virtual void FilterContentBundlePaths(const FRequestUpdateContentBundleContext& BundleContext, TArray<FString>& InOutContentPaths) {}

protected:
	FTSTicker::FDelegateHandle TickHandle;

	enum class EAsyncInitStep : int
	{
		None,
		MakeBundlesForBulkBuild,
		Finishing,
		Count,
	};
	friend const TCHAR* LexToString(FInstallBundleSourceBulk::EAsyncInitStep Val);

	enum class EAsyncInitStepResult : int
	{
		Waiting,
		Done,
	};

	EInstallBundleManagerInitState InitState = EInstallBundleManagerInitState::NotInitialized;
	EInstallBundleManagerInitResult InitResult = EInstallBundleManagerInitResult::OK;
	EAsyncInitStep InitStep = EAsyncInitStep::None;
	EAsyncInitStep LastInitStep = EAsyncInitStep::None;
	EAsyncInitStepResult InitStepResult = EAsyncInitStepResult::Done;
	bool bRetryInit = false;
	FInstallBundleSourceInitDelegate OnInitCompleteCallback;
	TArray<TUniquePtr<InstallBundleUtil::FInstallBundleTask>> InitAsyncTasks;

	TMap<FName, TArray<FString>> BulkBuildBundles; // BundleName -> Files

	TSharedPtr<IAnalyticsProviderET> AnalyticsProvider;
};

#undef UE_API
