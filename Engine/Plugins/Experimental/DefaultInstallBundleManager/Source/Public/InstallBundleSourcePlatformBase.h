// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "InstallBundleTypes.h"
#include "InstallBundleManagerUtil.h"

#if WITH_PLATFORM_INSTALL_BUNDLE_SOURCE || WITH_PLATFORM_DLC_INSTALL_BUNDLE_SOURCE
#include "InstallBundleSourceInterface.h"
#include "Internationalization/Regex.h"


class FInstallBundleSourcePlatformBase : public IInstallBundleSource
{
private:
public:
	DEFAULTINSTALLBUNDLEMANAGER_API FInstallBundleSourcePlatformBase();
	FInstallBundleSourcePlatformBase(const FInstallBundleSourcePlatformBase& Other) = delete;
	FInstallBundleSourcePlatformBase& operator=(const FInstallBundleSourcePlatformBase& Other) = delete;
	DEFAULTINSTALLBUNDLEMANAGER_API virtual ~FInstallBundleSourcePlatformBase();


	// IInstallBundleSource Interface
public:
	DEFAULTINSTALLBUNDLEMANAGER_API virtual FInstallBundleSourceType GetSourceType() const override;

	DEFAULTINSTALLBUNDLEMANAGER_API virtual FInstallBundleSourceInitInfo Init(
		TSharedRef<InstallBundleUtil::FContentRequestStatsMap> InRequestStats,
		TSharedPtr<IAnalyticsProviderET> AnalyticsProvider,
		TSharedPtr<InstallBundleUtil::PersistentStats::FPersistentStatContainerBase> InPersistentStatsContainer) override;


	DEFAULTINSTALLBUNDLEMANAGER_API virtual void AsyncInit_QueryBundleInfo(FInstallBundleSourceQueryBundleInfoDelegate Callback) override;

	DEFAULTINSTALLBUNDLEMANAGER_API virtual EInstallBundleManagerInitState GetInitState() const override;

	DEFAULTINSTALLBUNDLEMANAGER_API virtual FString GetContentVersion() const override;

	DEFAULTINSTALLBUNDLEMANAGER_API virtual TSet<FName> GetBundleDependencies(FName InBundleName, TSet<FName>* SkippedUnknownBundles /*= nullptr*/) const override;

	virtual void SetAnalyticsProvider(TSharedPtr<IAnalyticsProviderET> NewAnalyticsProvider) override
	{
		AnalyticsProvider = NewAnalyticsProvider;
	}

protected:
	virtual bool QueryPersistentBundleInfo(FInstallBundleSourcePersistentBundleInfo& SourceBundleInfo) const = 0;

	template<typename T>
	inline void StatsBegin(const InstallBundleUtil::FContentRequestStatsKey& Key, T State)
	{
		RequestStats->StatsBegin(Key, LexToString(State));
	}
	
	template<typename T>
	inline void StatsEnd(const InstallBundleUtil::FContentRequestStatsKey& Key, T State, uint64 DataSize = 0)
	{
		RequestStats->StatsEnd(Key, LexToString(State), DataSize);
	}

	TSharedPtr<IAnalyticsProviderET> AnalyticsProvider;

	TSharedPtr<InstallBundleUtil::FContentRequestStatsMap> RequestStats;

	TSharedPtr<InstallBundleUtil::PersistentStats::FPersistentStatContainerBase> PersistentStatsContainer;

	EInstallBundleManagerInitState InitState = EInstallBundleManagerInitState::NotInitialized;

	TArray<TPair<FString, TArray<FRegexPattern>>> BundleRegexList; // BundleName -> FileRegex
	TArray<TPair<FString, TArray<FRegexPattern>>> BundleSubsetRegexList;
};
#endif //WITH_PLATFORM_INSTALL_BUNDLE_SOURCE || WITH_PLATFORM_DLC_INSTALL_BUNDLE_SOURCE
