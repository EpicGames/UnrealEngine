// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "InstallBundleSourcePlatformBase.h"

#if WITH_PLATFORM_INSTALL_BUNDLE_SOURCE
#include "Containers/StaticArray.h"
#include "Containers/Ticker.h"
#include "GenericPlatform/GenericPlatformChunkInstall.h"

class FInstallBundleSourcePlatformChunkInstall : public FInstallBundleSourcePlatformBase
{
private:

	// Internal Types
	struct FBundleInfo
	{
	public:
		EInstallBundlePriority Priority = EInstallBundlePriority::Low;

		FName NamedChunk;
		TArray<FString> FilePaths;
	};
	

	class FContentRequest
	{
	public:
		FName BundleName;
		ELogVerbosity::Type LogVerbosityOverride = ELogVerbosity::NoLogging;
		bool bInProgress = true;
		bool bCancelled = false;

		TArray<FString> ContentPaths;
		FInstallBundleCompleteDelegate CompleteCallback;
	};
	using FContentRequestRef = TSharedRef<FContentRequest, ESPMode::ThreadSafe>;
	using FContentRequestPtr = TSharedPtr<FContentRequest, ESPMode::ThreadSafe>;
	using FContentRequestWeakPtr = TWeakPtr<FContentRequest, ESPMode::ThreadSafe>;

	class FContentReleaseRequest
	{
	public:
		FName BundleName;
		ELogVerbosity::Type LogVerbosityOverride = ELogVerbosity::NoLogging;
		bool bInProgress = true;
		bool bFailed = false;
		
		FInstallBundleRemovedDelegate CompleteCallback;
	};
	using FContentReleaseRequestRef = TSharedRef<FContentReleaseRequest, ESPMode::ThreadSafe>;
	using FContentReleaseRequestPtr = TSharedPtr<FContentReleaseRequest, ESPMode::ThreadSafe>;
	using FContentReleaseRequestWeakPtr = TWeakPtr<FContentReleaseRequest, ESPMode::ThreadSafe>;

	
public:
	DEFAULTINSTALLBUNDLEMANAGER_API FInstallBundleSourcePlatformChunkInstall( IPlatformChunkInstall* InPlatformChunkInstall);
	FInstallBundleSourcePlatformChunkInstall(const FInstallBundleSourcePlatformChunkInstall& Other) = delete;
	FInstallBundleSourcePlatformChunkInstall& operator=(const FInstallBundleSourcePlatformChunkInstall& Other) = delete;
	DEFAULTINSTALLBUNDLEMANAGER_API virtual ~FInstallBundleSourcePlatformChunkInstall();

private:

	DEFAULTINSTALLBUNDLEMANAGER_API bool Tick(float dt);

	DEFAULTINSTALLBUNDLEMANAGER_API void TickUpdateChunkOrder();

	// IInstallBundleSource Interface
public:
	DEFAULTINSTALLBUNDLEMANAGER_API virtual FInstallBundleSourceInitInfo Init(
		TSharedRef<InstallBundleUtil::FContentRequestStatsMap> InRequestStats,
		TSharedPtr<IAnalyticsProviderET> AnalyticsProvider,
		TSharedPtr<InstallBundleUtil::PersistentStats::FPersistentStatContainerBase> InPersistentStatsContainer) override;

	DEFAULTINSTALLBUNDLEMANAGER_API virtual void AsyncInit(FInstallBundleSourceInitDelegate Callback) override;

	DEFAULTINSTALLBUNDLEMANAGER_API virtual void GetContentState(TArrayView<const FName> BundleNames, EInstallBundleGetContentStateFlags Flags, FInstallBundleGetContentStateDelegate Callback) override;

	DEFAULTINSTALLBUNDLEMANAGER_API virtual EInstallBundleSourceBundleSkipReason GetBundleSkipReason(FName BundleName) const override;

	DEFAULTINSTALLBUNDLEMANAGER_API virtual void RequestUpdateContent(FRequestUpdateContentBundleContext Context) override;

	DEFAULTINSTALLBUNDLEMANAGER_API virtual void RequestReleaseContent(FRequestReleaseContentBundleContext BundleContext) override;

	DEFAULTINSTALLBUNDLEMANAGER_API virtual void CancelBundles(TArrayView<const FName> BundleNames) override;

	DEFAULTINSTALLBUNDLEMANAGER_API virtual TOptional<FInstallBundleSourceProgress> GetBundleProgress(FName BundleName) const override;

protected:
	DEFAULTINSTALLBUNDLEMANAGER_API virtual bool QueryPersistentBundleInfo(FInstallBundleSourcePersistentBundleInfo& SourceBundleInfo) const override;

	DEFAULTINSTALLBUNDLEMANAGER_API void OnNamedChunkInstall(const FNamedChunkCompleteCallbackParam& Param);

	DEFAULTINSTALLBUNDLEMANAGER_API FName GetNamedChunkForBundle(FName BundleName) const;

private:
	FTSTicker::FDelegateHandle TickHandle;
	FDelegateHandle NamedChunkInstallDelegateHandle;

	TMap<FName, FBundleInfo> BundleInfoMap;
	TSet<FName> NamedChunks;

	TArray<FContentRequestRef> ContentRequests;
	TArray<FContentReleaseRequestRef> ContentReleaseRequests;

	TArray<TUniquePtr<InstallBundleUtil::FInstallBundleTask>> GeneralAsyncTasks;

	IPlatformChunkInstall* PlatformChunkInstall;

	bool bChunkOrderDirty = false;
	bool bIsUpdatingChunkOrder = false;
};
#endif //WITH_PLATFORM_INSTALL_BUNDLE_SOURCE
