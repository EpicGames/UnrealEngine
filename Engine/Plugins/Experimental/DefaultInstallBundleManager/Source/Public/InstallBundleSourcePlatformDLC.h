// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "InstallBundleSourcePlatformBase.h"

#if WITH_PLATFORM_DLC_INSTALL_BUNDLE_SOURCE
#include "Containers/StaticArray.h"
#include "Containers/Ticker.h"
#include "PlatformDLC.h"

class FInstallBundleSourcePlatformDLC : public FInstallBundleSourcePlatformBase
{
private:

	// Internal Types
	struct FBundleInfo
	{
	public:
		EInstallBundlePriority Priority = EInstallBundlePriority::Low;

		FName DLCName;

		bool bHasSetFilePaths = false;
		bool bContainsIoStoreOnDemandToc = false;
		TArray<FString> FilePaths;
	};
	

	class FDLCContentRequest
	{
	public:
		FName BundleName;
		ELogVerbosity::Type LogVerbosityOverride = ELogVerbosity::NoLogging;
		bool bInProgress = true;
		bool bCancelled = false;

		FInstallBundleCompleteDelegate CompleteCallback;
	};
	using FDLCContentRequestRef = TSharedRef<FDLCContentRequest, ESPMode::ThreadSafe>;
	using FDLCContentRequestPtr = TSharedPtr<FDLCContentRequest, ESPMode::ThreadSafe>;
	using FDLCContentRequestWeakPtr = TWeakPtr<FDLCContentRequest, ESPMode::ThreadSafe>;

	class FDLCContentReleaseRequest
	{
	public:
		FName BundleName;
		ELogVerbosity::Type LogVerbosityOverride = ELogVerbosity::NoLogging;
		bool bInProgress = true;
		bool bFailed = false;
		bool bUninstallRequested = false;
		
		FInstallBundleRemovedDelegate CompleteCallback;
	};
	using FDLCContentReleaseRequestRef = TSharedRef<FDLCContentReleaseRequest, ESPMode::ThreadSafe>;
	using FDLCContentReleaseRequestPtr = TSharedPtr<FDLCContentReleaseRequest, ESPMode::ThreadSafe>;
	using FDLCContentReleaseRequestWeakPtr = TWeakPtr<FDLCContentReleaseRequest, ESPMode::ThreadSafe>;

	
public:
	DEFAULTINSTALLBUNDLEMANAGER_API FInstallBundleSourcePlatformDLC( TSharedPtr<IPlatformDLC> InPlatformDLC);
	FInstallBundleSourcePlatformDLC(const FInstallBundleSourcePlatformDLC& Other) = delete;
	FInstallBundleSourcePlatformDLC& operator=(const FInstallBundleSourcePlatformDLC& Other) = delete;
	DEFAULTINSTALLBUNDLEMANAGER_API virtual ~FInstallBundleSourcePlatformDLC();

	DEFAULTINSTALLBUNDLEMANAGER_API virtual FInstallBundleSourceType GetSourceType() const override;

private:

	DEFAULTINSTALLBUNDLEMANAGER_API bool Tick(float dt);

	// IInstallBundleSource Interface
public:
	DEFAULTINSTALLBUNDLEMANAGER_API virtual FInstallBundleSourceInitInfo Init(
		TSharedRef<InstallBundleUtil::FContentRequestStatsMap> InRequestStats,
		TSharedPtr<IAnalyticsProviderET> AnalyticsProvider,
		TSharedPtr<InstallBundleUtil::PersistentStats::FPersistentStatContainerBase> InPersistentStatsContainer) override;

	DEFAULTINSTALLBUNDLEMANAGER_API virtual void AsyncInit(FInstallBundleSourceInitDelegate Callback) override;

	DEFAULTINSTALLBUNDLEMANAGER_API virtual void GetContentState(TArrayView<const FName> BundleNames, EInstallBundleGetContentStateFlags Flags, FInstallBundleGetContentStateDelegate Callback) override;

	DEFAULTINSTALLBUNDLEMANAGER_API virtual EInstallBundleSourceBundleSkipReason GetBundleSkipReason(FName BundleName) const override;

	DEFAULTINSTALLBUNDLEMANAGER_API virtual void RequestUpdateContent(FRequestUpdateContentBundleContext Context) override;

	DEFAULTINSTALLBUNDLEMANAGER_API virtual void RequestReleaseContent(FRequestReleaseContentBundleContext BundleContext) override;

	DEFAULTINSTALLBUNDLEMANAGER_API virtual void CancelBundles(TArrayView<const FName> BundleNames) override;

	DEFAULTINSTALLBUNDLEMANAGER_API virtual TOptional<FInstallBundleSourceProgress> GetBundleProgress(FName BundleName) const override;

protected:
	DEFAULTINSTALLBUNDLEMANAGER_API virtual bool QueryPersistentBundleInfo(FInstallBundleSourcePersistentBundleInfo& SourceBundleInfo) const override;

	DEFAULTINSTALLBUNDLEMANAGER_API void OnDLCNotification(FName DLCName, IPlatformDLC::ENotification Notification, bool bSuccess);

	DEFAULTINSTALLBUNDLEMANAGER_API TArray<FString> GetContentPathsForDLC( FName DLCName, FName BundleName );

	DEFAULTINSTALLBUNDLEMANAGER_API FName GetDLCNameForBundle(FName BundleName) const;


	// Override to return the platform user to pass to SetStoreUser before each download.
	virtual FPlatformUserId GetDLCStoreUserId() const { return PLATFORMUSERID_NONE; }
private:
	FTSTicker::FDelegateHandle TickHandle;
	FDelegateHandle DLCNotificationDelegateHandle;

	TMap<FName, FBundleInfo> BundleInfoMap;
	TMap<FName, TArray<FString>> BundleToSubsetMap;
	TSet<FName> DLCNames;
	TArray<FString> PakSearchDirs;

	TArray<FDLCContentRequestRef> ContentRequests;
	TArray<FDLCContentReleaseRequestRef> ContentReleaseRequests;

	TArray<TUniquePtr<InstallBundleUtil::FInstallBundleTask>> GeneralAsyncTasks;

	TSharedPtr<IPlatformDLC> PlatformDLC;
};
#endif //WITH_PLATFORM_DLC_INSTALL_BUNDLE_SOURCE
